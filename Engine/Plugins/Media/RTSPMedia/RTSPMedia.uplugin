{
    "FileVersion": 3,
    "Version": 1,
    "VersionName": "1.0",
    "FriendlyName": "RTSP Media",
    "Description": "Real-time media streaming via the RTSP protocol",
    "Category": "Media Players",
    "CreatedBy": "Epic Games",
    "CreatedByURL": "https://epicgames.com",
    "CanContainContent": true,
    "IsExperimentalVersion": true,
    "EnabledByDefault": false,
    "SupportedTargetPlatforms": [
        "Win64",
        "Mac",
        "Linux"
    ],
    "Modules": [
        {
            "Name": "RTSPMedia",
            "Type": "Runtime",
            "LoadingPhase": "Default"
        },
        {
            "Name": "RTSPMediaEditor",
            "Type": "Editor",
            "LoadingPhase": "PostEngineInit"
        }
    ],
    "Plugins": [
        {
            "Name": "ElectraCodecs",
            "Enabled": true
        },
        {
            "Name": "MediaIOFramework",
            "Enabled": true
        },
        {
            "Name": "MediaPlayerEditor",
            "Enabled": true
        }
    ]
}