// Copyright Epic Games, Inc. All Rights Reserved.

#include "Player/RtspMediaPlayer.h"

// RTSPMedia
#include "RTP/RtpDecoder.h"
#include "RTP/RtpPacket.h"
#include "RtspMediaConstants.h"
#include "RtspMediaDefaults.h"
#include "RtspMediaSource.h"
#include "SDP/SdpSession.h"

#include "CoreMinimal.h"

// Media Framework
#include "IMediaEventSink.h"
#include "IMediaOptions.h"
#include "MediaSamples.h"
#include "Async/Async.h"

FRtspMediaPlayer::FRtspMediaPlayer(IMediaEventSink& InEventSink)
	: Super(InEventSink)
	// We always need a Samples object, even if we replace it later
	// FRtspMediaPlayer::GetSamples returns a reference to a FMediaSamples instance
	, Samples(new FMediaSamples(0, 0))
{}

FRtspMediaPlayer::~FRtspMediaPlayer()
{
	FRtspMediaPlayer::Close();
}

// UsePlaybackTimingV2 is essential here, it dictates which overriden methods are called to trigger the gathering of samples.
bool FRtspMediaPlayer::GetPlayerFeatureFlag(EFeatureFlag InFlag) const
{
	if (InFlag == EFeatureFlag::AlwaysPullNewestVideoFrame ||
		InFlag == EFeatureFlag::AllowShutdownOnClose ||
		InFlag == EFeatureFlag::UsePlaybackTimingV2)
	{
		return true;
	}

	return false;
}

FGuid FRtspMediaPlayer::GetPlayerPluginGUID() const
{
	return RtspMedia::PlayerPluginGUID;
}

FTimespan FRtspMediaPlayer::GetTime() const
{
	// When using V2 timing the media framework uses the timestamp within the media samples
	return FTimespan::Zero();
}

bool FRtspMediaPlayer::Open(const FString& InUrl, const IMediaOptions* InOptions)
{
	if (!ensureMsgf(InOptions != nullptr, TEXT("Media player open was attempted but no options were provided.")))
	{
		return false;
	}
	
	CurrentUrl = InUrl;
	
	UE_LOGF(LogRtspMedia, Verbose, "Reading options");
	const int64 MaxQueuedVideoSamplesOption = InOptions->GetMediaOption(RtspMedia::Option::MaxQueuedVideoSamples, static_cast<int64>(RtspMedia::Default::MaxQueuedVideoSamples));
	Samples = MakeUnique<FMediaSamples>(RtspMedia::Default::MaxQueuedAudioSamples, static_cast<int32>(FMath::Clamp(MaxQueuedVideoSamplesOption, 2, 16)));

	const double RequestTimeoutSecondsOption = InOptions->GetMediaOption(RtspMedia::Option::RequestTimeoutSeconds, RtspMedia::Default::RequestTimeoutSeconds);
	const int64 SocketBufferSizeKbOption = InOptions->GetMediaOption(RtspMedia::Option::SocketBufferSize, static_cast<int64>(RtspMedia::Default::SocketBufferSizeKb));
	const int64 TransportProtocolOption = InOptions->GetMediaOption(RtspMedia::Option::TransportProtocol, static_cast<int64>(RtspMedia::Default::TransportProtocol));
	const int64 JitterBufferDepthMsOption = InOptions->GetMediaOption(RtspMedia::Option::JitterBufferDepthMs, static_cast<int64>(RtspMedia::Default::JitterBufferDepthMs));
	const bool bJitterBufferAutoAdjustOption = InOptions->GetMediaOption(RtspMedia::Option::JitterBufferAutoAdjust, RtspMedia::Default::bJitterBufferAutoAdjust);
	const double JitterBufferObservationWindowSecondsOption = InOptions->GetMediaOption(RtspMedia::Option::JitterBufferObservationWindowSeconds, RtspMedia::Default::JitterBufferObservationWindowSeconds);

	RtspClientConfiguration = MakeUnique<FRtspClientConfiguration>();
	RtspClientConfiguration->Url = InUrl;
	RtspClientConfiguration->RequestTimeoutSeconds = static_cast<float>(FMath::Clamp(RequestTimeoutSecondsOption, 1.0, 30.0));
	RtspClientConfiguration->SocketBufferSizeBytes = static_cast<int32>(FMath::Clamp(SocketBufferSizeKbOption, 128, 4096) * 1024);
	RtspClientConfiguration->TransportProtocol = static_cast<FRtspTransportConfiguration::TransportProtocol>(FMath::Clamp(TransportProtocolOption, 0, 1));
	RtspClientConfiguration->JitterBufferDepthMs = static_cast<uint32>(FMath::Clamp(JitterBufferDepthMsOption, 0, 5000));
	RtspClientConfiguration->bJitterBufferAutoAdjust = bJitterBufferAutoAdjustOption;
	RtspClientConfiguration->JitterBufferObservationWindowSeconds = static_cast<float>(FMath::Clamp(JitterBufferObservationWindowSecondsOption, 30.0, 300.0));

	ResetRestartState();
	bAutoRestart = InOptions->GetMediaOption(RtspMedia::Option::AutoReconnect, RtspMedia::Default::bAutoReconnect);
	const double MinReconnectDelaySecondsOption = InOptions->GetMediaOption(RtspMedia::Option::MinReconnectDelaySeconds, static_cast<double>(RtspMedia::Default::MinReconnectDelaySeconds));
	MinRestartDelaySeconds = static_cast<float>(FMath::Clamp(MinReconnectDelaySecondsOption, 1.0, 60.0));
	const double MaxReconnectDelaySecondsOption = InOptions->GetMediaOption(RtspMedia::Option::MaxReconnectDelaySeconds, static_cast<double>(RtspMedia::Default::MaxReconnectDelaySeconds));
	MaxRestartDelaySeconds = static_cast<float>(FMath::Clamp(MaxReconnectDelaySecondsOption, 1.0, 300.0));
	const int64 MaxReconnectAttemptsOption = InOptions->GetMediaOption(RtspMedia::Option::MaxReconnectAttempts, static_cast<int64>(RtspMedia::Default::MaxReconnectAttempts));
	MaxRestartAttempts = static_cast<int32>(FMath::Clamp(MaxReconnectAttemptsOption, 0, 1000));

	// Decoder options
	MaxFragmentBufferSizeMbOption = InOptions->GetMediaOption(RtspMedia::Option::MaxFragmentBufferSize, static_cast<int64>(RtspMedia::Default::MaxFragmentBufferSizeMb));
	DecoderBufferSizeOption = InOptions->GetMediaOption(RtspMedia::Option::DecoderBufferSize, static_cast<int64>(RtspMedia::Default::DecoderBufferSize));
	DecoderPollIntervalMsOption = InOptions->GetMediaOption(RtspMedia::Option::DecoderPollIntervalMs, static_cast<int64>(RtspMedia::Default::DecoderPollIntervalMs));
	bProvideCpuBufferOption = InOptions->GetMediaOption(RtspMedia::Option::ProvideCpuBuffer, RtspMedia::Default::bProvideCpuBuffer);

	ResetClient();
	ResetDecoder();

	// InitializeClient() may fail synchronously (e.g., Connect() fails immediately), but we still return true at the end of this method to indicate we've accepted the URL.
	// Returning false would cause the Media Framework to abandon this player, preventing auto-reconnect from taking place via TickInput().
	InitializeClient();
	
	return true;
}

void FRtspMediaPlayer::TickInput(FTimespan InDeltaTime, FTimespan InTimecode)
{
	if (bAutoRestart)
	{
		if (RestartCountdownSeconds > 0.0f)
		{
			RestartCountdownSeconds -= static_cast<float>(InDeltaTime.GetTotalSeconds());

			if (RestartCountdownSeconds <= 0.0f)
			{
				RestartCountdownSeconds = 0.0f;
				ResetClient();
				InitializeClient();
			}
		}
	}
	
	FStreamingMediaPlayerBase::TickInput(InDeltaTime, InTimecode);
}

void FRtspMediaPlayer::Close()
{
	if (CurrentState == EMediaState::Closed)
	{
		return;
	}
	
	UE_LOGF(LogRtspMedia, Log, "Closing media player");
	
	ResetRestartState();

	CurrentState = EMediaState::Closed;
	EventSink.ReceiveMediaEvent(EMediaEvent::MediaClosed);

	if (Samples.IsValid())
	{
		// Don't reset Samples, the media player always needs a valid reference until the player is destroyed
		Samples->FlushSamples();
	}

	ResetClient();
	ResetDecoder();
}

IMediaSamples& FRtspMediaPlayer::GetSamples()
{
	return *Samples;
}

bool FRtspMediaPlayer::GetVideoTrackFormat(int32 InTrackIndex, int32 InFormatIndex, FMediaVideoTrackFormat& OutFormat) const
{
	if (InTrackIndex == 0 && InFormatIndex == 0 && NumVideoTracks > 0)
	{
		const float FrameRate = RtpDecoder.IsValid() ? static_cast<float>(RtpDecoder->GetFrameRate()) : 30.0f;
		OutFormat.Dim = FIntPoint(VideoFormatWidth, VideoFormatHeight);
		OutFormat.FrameRate = FrameRate;
		OutFormat.FrameRates = TRange<float>(FrameRate);
		OutFormat.TypeName = TEXT("H.264");
		return true;
	}

	return false;
}

void FRtspMediaPlayer::HandleOnClientStateChanged(ERtspClientState InNewState, ERtspClientState InOldState)
{
	switch (InNewState)
	{
		case ERtspClientState::Playing:
			if (!RtpDecoder.IsValid() || !RtpDecoder->Start())
			{
				UE_LOGF(LogRtspMedia, Error, "Failed to start RTP decoder");
				SetState(EMediaState::Error);
				EventSink.ReceiveMediaEvent(EMediaEvent::MediaOpenFailed);
				// If the decoder fails to start we intentionally close the media source rather than trigger a retry.
				Close();
			}
			else
			{
				CurrentStatus = EMediaStatus::None;
				CurrentState = EMediaState::Playing;
				EventSink.ReceiveMediaEvent(EMediaEvent::PlaybackResumed);
				ResetRestartState();
			}
			break;
		case ERtspClientState::Setup:
			EventSink.ReceiveMediaEvent(EMediaEvent::MediaOpened);
			break;
		case ERtspClientState::Error:
			HandlePlayerError();
			break;
		default:
			break;
	}
}

// This method is called by RtspClient on its dedicated worker thread to avoid a game thread hop.
void FRtspMediaPlayer::HandleOnRtpPacketReceived(FRtpPacket InPacket)
{
	if (!RtpDecoder.IsValid())
	{
		UE_LOGF(LogRtspMedia, Verbose, "RTP decoder was not available when packet was received");
		return;
	}
	
	RtpDecoder->EnqueuePacket(MoveTemp(InPacket));
}

void FRtspMediaPlayer::HandleOnSdpSessionReceived(const FSdpSession& Session)
{
	UE_LOGF(LogRtspMedia, Verbose, "SDP received");

	if (!Session.HasVideo())
	{
		UE_LOGF(LogRtspMedia, Error, "No video track present within SDP session");
		HandlePlayerError();
		return;
	}

	// We only support a single video track at this time
	NumVideoTracks = 1;

	// Ensure the media framework is informed now we have a video track available.
	EventSink.ReceiveMediaEvent(EMediaEvent::TracksChanged);
	
	// TODO: Signal audio track availability once audio is implemented

	// Reset decoder in case we already have one at this point
	ResetDecoder();
	
	// Create the decoder
	const int64 MaxFragmentBufferSize = FMath::Clamp(MaxFragmentBufferSizeMbOption, 2, 64) * 1024 * 1024;
	const int64 DecoderBufferSize = FMath::Clamp(DecoderBufferSizeOption, 1, 16);
	const uint32 DecoderPollIntervalMs = static_cast<uint32>(FMath::Clamp(DecoderPollIntervalMsOption, 1, 10));
	RtpDecoder = MakeUnique<FRtpDecoder>();
	if (!RtpDecoder->Initialize(
		Samples.Get(),
		Session,
		MaxFragmentBufferSize,
		DecoderBufferSize,
		DecoderPollIntervalMs,
		bProvideCpuBufferOption))
	{
		UE_LOGF(LogRtspMedia, Error, "Failed to initialise decoder");
		SetState(EMediaState::Error);
		EventSink.ReceiveMediaEvent(EMediaEvent::MediaOpenFailed);
		Close();
	}
	else
	{
		// Bind to decoder for video dimensions information and for error handling
		RtpDecoder->OnVideoDimensions.BindSP(this, &FRtspMediaPlayer::HandleOnVideoDimensions);
		RtpDecoder->OnDecoderError.BindSP(this, &FRtspMediaPlayer::HandleOnDecoderError);
	}
	
}

void FRtspMediaPlayer::HandleOnVideoDimensions(int32 InWidth, int32 InHeight)
{
	UE_LOGF(LogRtspMedia, Verbose, "Player was provided with updated video dimensions. Width: %d Height: %d", InWidth, InHeight);

	TWeakPtr<FRtspMediaPlayer> WeakSelf = AsWeak();
	AsyncTask(ENamedThreads::GameThread, [WeakSelf, InWidth, InHeight]
	{
		const TSharedPtr<FRtspMediaPlayer> Player = WeakSelf.Pin();
		if (Player.IsValid())
		{
			Player->VideoFormatWidth = InWidth;
			Player->VideoFormatHeight = InHeight;
		}
	});
}

void FRtspMediaPlayer::HandleOnDecoderError()
{
	TWeakPtr<FRtspMediaPlayer> WeakSelf = AsWeak();
	AsyncTask(ENamedThreads::GameThread, [WeakSelf]
	{
		const TSharedPtr<FRtspMediaPlayer> Player = WeakSelf.Pin();
		if (Player.IsValid())
		{
			Player->HandlePlayerError();
		}
	});
}

void FRtspMediaPlayer::ResetClient()
{
	if (RtspClient.IsValid())
	{
		RtspClient->OnStateChanged.Unbind();
		RtspClient->OnSdpSessionReceived.Unbind();
		RtspClient->OnRtpPacketReceived.Unbind();

		RtspClient->Disconnect();
		RtspClient.Reset();
	}
}

void FRtspMediaPlayer::ResetDecoder()
{
	if (RtpDecoder.IsValid())
	{
		RtpDecoder->OnDecoderError.Unbind();
		RtpDecoder->OnVideoDimensions.Unbind();
		RtpDecoder->Shutdown();
		RtpDecoder.Reset();
	}
}

void FRtspMediaPlayer::ResetRestartState()
{
	RestartAttempts = 0;
	RestartCountdownSeconds = 0.0f;
}

void FRtspMediaPlayer::InitializeClient()
{
	UE_LOGF(LogRtspMedia, Verbose, "Initialising RTSP Client");

	RtspClient = MakeUnique<FRtspClient>();

	RtspClient->OnSdpSessionReceived.BindSP(this, &FRtspMediaPlayer::HandleOnSdpSessionReceived);
	RtspClient->OnStateChanged.BindSP(this, &FRtspMediaPlayer::HandleOnClientStateChanged);
	RtspClient->OnRtpPacketReceived.BindSP(this, &FRtspMediaPlayer::HandleOnRtpPacketReceived);

	if (RtspClient->Initialize(*RtspClientConfiguration))
	{
		CurrentStatus = EMediaStatus::Connecting;
		CurrentState = EMediaState::Preparing;
		EventSink.ReceiveMediaEvent(EMediaEvent::MediaConnecting);
	}
	else
	{
		CurrentStatus = EMediaStatus::None;
		CurrentState = EMediaState::Error;
		EventSink.ReceiveMediaEvent(EMediaEvent::MediaOpenFailed);
	}
}

void FRtspMediaPlayer::HandlePlayerError()
{
	if (CurrentState == EMediaState::Closed)
	{
		return;
	}
	
	ResetClient();
	
	if (bAutoRestart && (MaxRestartAttempts == 0 || RestartAttempts < MaxRestartAttempts))
	{
		ScheduleRestart();
		return;
	}

	if (RestartAttempts >= MaxRestartAttempts)
	{
		UE_LOGF(LogRtspMedia, Warning, "Reached the max number of client restart attempts (%d). Stopping.", MaxRestartAttempts);
	}
	
	CurrentStatus = EMediaStatus::None;
	CurrentState = EMediaState::Error;
	EventSink.ReceiveMediaEvent(EMediaEvent::MediaOpenFailed);
}

// Calculate how long we should wait until the next retry attempt
// We cap the retry period at MaxRestartDelaySeconds and introduce
// a small random offset in case multiple clients all lose their
// connection at the same time to avoid spamming the server simultaneously. 
void FRtspMediaPlayer::ScheduleRestart()
{
	// How quickly do we want to reach the maximum restart delay?
	// If we have a limited number of retry attempts, use that value,
	// otherwise we'll hit the max retry period after 10.0 attempts.
	const float ScaleFactor = MaxRestartAttempts > 0 ? static_cast<float>(MaxRestartAttempts) : 10.0f;
	// Normalised progress towards max retry delay
	const float Progress = FMath::Min(static_cast<float>(RestartAttempts) / ScaleFactor, 1.0f);
	// Cubic interpolation so that the first few delays stay low then ramp up 
	const float Delay = FMath::InterpEaseIn(MinRestartDelaySeconds, MaxRestartDelaySeconds, Progress, 3.0f);

	// Applying a small random offset to the delay so that multiple sources don't all hammer the server at once 
	RestartCountdownSeconds = Delay + FMath::FRandRange(0.0f, 1.0f);
	RestartAttempts++;

	UE_LOGF(LogRtspMedia, Log, "Restarting RTSP client in %.1f seconds. Attempt number %d", RestartCountdownSeconds, RestartAttempts);

	CurrentStatus = EMediaStatus::Connecting;
	CurrentState = EMediaState::Preparing;
	EventSink.ReceiveMediaEvent(EMediaEvent::MediaConnecting);
}
