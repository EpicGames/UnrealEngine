// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Player/StreamingMediaPlayerBase.h"

#include "RTSP/RtspClient.h"

#include "RTP/RtpDecoder.h"
#include "Templates/SharedPointer.h"

class FRtpDecoder;
struct FRtpPacket;

class FMediaSamples;

class FRtspMediaPlayer 
	: public FStreamingMediaPlayerBase
	, public TSharedFromThis<FRtspMediaPlayer, ESPMode::ThreadSafe>
{
	using Super = FStreamingMediaPlayerBase;

public:
	FRtspMediaPlayer(IMediaEventSink& InEventSink);
	virtual ~FRtspMediaPlayer() override;

	//~ Begin IMediaPlayer
	virtual void Close() override;
	virtual IMediaSamples& GetSamples() override;
	virtual bool GetPlayerFeatureFlag(EFeatureFlag InFlag) const override;
	virtual FGuid GetPlayerPluginGUID() const override;
	virtual FTimespan GetTime() const override;
	virtual bool Open(const FString& InUrl, const IMediaOptions* InOptions) override;
	virtual void TickInput(FTimespan InDeltaTime, FTimespan InTimecode) override;
	//~ End IMediaPlayer

	// Overridden from FStreamingMediaPlayerBase
	virtual bool GetVideoTrackFormat(int32 InTrackIndex, int32 InFormatIndex, FMediaVideoTrackFormat& OutFormat) const override;

private:
	
	void HandleOnClientStateChanged(ERtspClientState InNewState, ERtspClientState InOldState);
	void HandleOnRtpPacketReceived(FRtpPacket InPacket);
	void HandleOnSdpSessionReceived(const FSdpSession& InSession);
	void HandleOnVideoDimensions(int32 InWidth, int32 InHeight);
	void HandleOnDecoderError();

	void ResetClient();
	void ResetDecoder();
	void ResetRestartState();

	void InitializeClient();
	void HandlePlayerError();
	void ScheduleRestart();
	
	// RTSP Networking
	TUniquePtr<FRtspClientConfiguration> RtspClientConfiguration;
	TUniquePtr<FRtspClient> RtspClient;

	// Decoding
	int64 MaxFragmentBufferSizeMbOption = RtspMedia::Default::MaxFragmentBufferSizeMb;
	int64 DecoderBufferSizeOption = RtspMedia::Default::DecoderBufferSize;
	int64 DecoderPollIntervalMsOption = RtspMedia::Default::DecoderPollIntervalMs;
	bool bProvideCpuBufferOption = RtspMedia::Default::bProvideCpuBuffer;
	TUniquePtr<FRtpDecoder> RtpDecoder;
	
	// Media Player
	TUniquePtr<FMediaSamples> Samples;
	int32 VideoFormatWidth = 0;
	int32 VideoFormatHeight = 0;

	// Auto Restart Configuration
	bool bAutoRestart = RtspMedia::Default::bAutoReconnect;
	float MinRestartDelaySeconds = RtspMedia::Default::MinReconnectDelaySeconds;
	float MaxRestartDelaySeconds = RtspMedia::Default::MaxReconnectDelaySeconds;
	int32 MaxRestartAttempts = RtspMedia::Default::MaxReconnectAttempts;

	// Restart State
	int32 RestartAttempts = 0;
	float RestartCountdownSeconds = 0.0f;
};
