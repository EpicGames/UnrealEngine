// Copyright Epic Games, Inc. All Rights Reserved.

#include "Player/StreamingMediaPlayerBase.h"

#include "IMediaEventSink.h"

#define LOCTEXT_NAMESPACE "StreamMediaPlayerBase"

FStreamingMediaPlayerBase::FStreamingMediaPlayerBase(IMediaEventSink& InEventSink)
	: EventSink(InEventSink)
{}

void FStreamingMediaPlayerBase::SetState(EMediaState InNewState)
{
	if (CurrentState != InNewState)
	{
		CurrentState = InNewState;
	}
}

//~ Begin IMediaPlayer

IMediaCache& FStreamingMediaPlayerBase::GetCache()
{
	return *this;
}

IMediaControls& FStreamingMediaPlayerBase::GetControls()
{
	return *this;
}

FString FStreamingMediaPlayerBase::GetInfo() const
{
	// Subclass could override to display relevant debug info
	return FString();
}

FString FStreamingMediaPlayerBase::GetStats() const
{
	// Subclass could override to display stats
	return FString();
}

IMediaTracks& FStreamingMediaPlayerBase::GetTracks()
{
	return *this;
}

FString FStreamingMediaPlayerBase::GetUrl() const
{
	return CurrentUrl;
}

IMediaView& FStreamingMediaPlayerBase::GetView()
{
	return *this;
}

bool FStreamingMediaPlayerBase::Open(const TSharedRef<FArchive, ESPMode::ThreadSafe>& InArchive, const FString& InOriginalUrl, const IMediaOptions* InOptions)
{
	// We don't support opening from archives.
	return false;
}

//~ End IMediaPlayer

//~ Begin IMediaControls

bool FStreamingMediaPlayerBase::CanControl(EMediaControl InControl) const 
{
	// We won't offer pause, seeking or scrub for live streams
	return false;
}

FTimespan FStreamingMediaPlayerBase::GetDuration() const 
{
	return FTimespan::MaxValue();
}

float FStreamingMediaPlayerBase::GetRate() const 
{
	// Normal speed
	return 1.0f;
}

EMediaState FStreamingMediaPlayerBase::GetState() const 
{
	return CurrentState;
}

EMediaStatus FStreamingMediaPlayerBase::GetStatus() const 
{
	// Subclasses can override to provide status information
	return CurrentStatus;
}

TRangeSet<float> FStreamingMediaPlayerBase::GetSupportedRates(EMediaRateThinning InThinning) const 
{
	// We only support a single rate, normal speed.
	TRangeSet<float> Rates;
	Rates.Add(TRange<float>(1.0f));
	return Rates;
}

bool FStreamingMediaPlayerBase::IsLooping() const 
{
	return false;
}

bool FStreamingMediaPlayerBase::Seek(const FTimespan& InTime) 
{
	return false;
}

bool FStreamingMediaPlayerBase::SetLooping(bool bInLooping) 
{
	return false;
}

bool FStreamingMediaPlayerBase::SetRate(float InRate) 
{
	// Only normal speed (1.0) is accepted
	return FMath::IsNearlyEqual(InRate, 1.0f);
}

//~ End IMediaControls

//~ Begin IMediaTracks

bool FStreamingMediaPlayerBase::GetAudioTrackFormat(int32 InTrackIndex, int32 InFormatIndex, FMediaAudioTrackFormat& OutFormat) const
{
	// Subclasses override to provide track format information
	return false;
}

int32 FStreamingMediaPlayerBase::GetNumTracks(EMediaTrackType InTrackType) const
{
	switch (InTrackType)
	{
		case EMediaTrackType::Video: return NumVideoTracks;
		case EMediaTrackType::Audio: return NumAudioTracks;
		default: return 0;
	}
}

int32 FStreamingMediaPlayerBase::GetNumTrackFormats(EMediaTrackType InTrackType, int32 InTrackIndex) const
{
	// We only support a single format for the track types we support
	if (InTrackIndex != 0)
	{
		return 0;
	}
	switch (InTrackType)
	{
		case EMediaTrackType::Video: return NumVideoTracks > 0 ? 1 : 0;
		case EMediaTrackType::Audio: return NumAudioTracks > 0 ? 1 : 0;
		default: return 0;
	}
}

int32 FStreamingMediaPlayerBase::GetSelectedTrack(EMediaTrackType InTrackType) const
{
	switch (InTrackType)
	{
		case EMediaTrackType::Video: return SelectedVideoTrack;
		case EMediaTrackType::Audio: return SelectedAudioTrack;
		default: return INDEX_NONE;
	}
}

FText FStreamingMediaPlayerBase::GetTrackDisplayName(EMediaTrackType InTrackType, int32 InTrackIndex) const
{
	switch (InTrackType)
	{
		case EMediaTrackType::Video: return NumVideoTracks > 0 ? LOCTEXT("VideoTrackDisplayName", "Video") : FText();
		case EMediaTrackType::Audio: return NumAudioTracks > 0 ? LOCTEXT("AudioTrackDisplayName", "Audio") : FText();
		default: return FText();
	}
}

int32 FStreamingMediaPlayerBase::GetTrackFormat(EMediaTrackType InTrackType, int32 InTrackIndex) const
{
	if (InTrackIndex == 0)
	{
		if (InTrackType == EMediaTrackType::Video && NumVideoTracks > 0)
		{
			return 0;
		}
		else if (InTrackType == EMediaTrackType::Audio && NumAudioTracks > 0)
		{
			return 0;
		}
	}
	return INDEX_NONE;
}

FString FStreamingMediaPlayerBase::GetTrackLanguage(EMediaTrackType InTrackType, int32 InTrackIndex) const
{
	return FString();
}

FString FStreamingMediaPlayerBase::GetTrackName(EMediaTrackType InTrackType, int32 InTrackIndex) const
{
	switch (InTrackType)
	{
		case EMediaTrackType::Video: return NumVideoTracks > 0 ? TEXT("video") : FString();
		case EMediaTrackType::Audio: return NumAudioTracks > 0 ? TEXT("audio") : FString();
		default: return FString();
	}
}

bool FStreamingMediaPlayerBase::GetVideoTrackFormat(int32 InTrackIndex, int32 InFormatIndex, FMediaVideoTrackFormat& OutFormat) const
{
	// Subclasses override to provide video track information
	return false;
}

bool FStreamingMediaPlayerBase::SelectTrack(EMediaTrackType InTrackType, int32 InTrackIndex)
{
	if (InTrackIndex < 0 && InTrackIndex != INDEX_NONE)
	{
		return false;
	}
	
	switch (InTrackType)
	{
		case EMediaTrackType::Video:
			if (InTrackIndex >= NumVideoTracks)
			{
				return false;
			}
			SelectedVideoTrack = InTrackIndex;
			return true;
		case EMediaTrackType::Audio: 
			if (InTrackIndex >= NumAudioTracks)
			{
				return false;
			}
			SelectedAudioTrack = InTrackIndex;
			return true;
		default: 
			return false;
	}
}

bool FStreamingMediaPlayerBase::SetTrackFormat(EMediaTrackType InTrackType, int32 InTrackIndex, int32 InFormatIndex)
{
	// We only support a single format for the track types we support
	if (InFormatIndex != 0)
	{
		return false;
	}

	// We only support one track of each type
	switch (InTrackType)
	{
		case EMediaTrackType::Video:
			return InTrackIndex == 0 && NumVideoTracks > 0;
		case EMediaTrackType::Audio:
			return InTrackIndex == 0 && NumAudioTracks > 0;
		default:
			return false;
	}
}

bool FStreamingMediaPlayerBase::SetVideoTrackFrameRate(int32 InTrackIndex, int32 InFormatIndex, float InFrameRate)
{
	// Can't control the frame rate from the client by default
	return false;
}

//~ End IMediaTracks

#undef LOCTEXT_NAMESPACE
