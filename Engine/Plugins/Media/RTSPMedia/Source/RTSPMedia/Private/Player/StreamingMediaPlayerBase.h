// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMediaPlayer.h"
#include "IMediaControls.h"
#include "IMediaTracks.h"
#include "IMediaCache.h"
#include "IMediaView.h"

class IMediaEventSink;

class FStreamingMediaPlayerBase
	: public IMediaPlayer
	, public IMediaControls
	, public IMediaTracks
	, public IMediaCache
	, public IMediaView
{
public:
	FStreamingMediaPlayerBase(IMediaEventSink &MediaEventSink);
	virtual ~FStreamingMediaPlayerBase() = default;

	virtual void Close() = 0;
	virtual FGuid GetPlayerPluginGUID() const = 0;
	virtual IMediaSamples& GetSamples() = 0;
	virtual bool Open(const FString& InUrl, const IMediaOptions* InOptions) = 0;

	//~ Begin IMediaPlayer
	IMediaCache& GetCache() override;
	IMediaControls& GetControls() override;
	FString GetInfo() const override;
	FString GetStats() const override;
	IMediaTracks& GetTracks() override;
	FString GetUrl() const override;
	IMediaView& GetView() override;
	bool Open(const TSharedRef<FArchive, ESPMode::ThreadSafe>& InArchive, const FString& InOriginalUrl, const IMediaOptions* InOptions) override;
	//~ End IMediaPlayer

	//~ Begin IMediaControls
	bool CanControl(EMediaControl InControl) const override;
	FTimespan GetDuration() const override;
	float GetRate() const override;
	EMediaState GetState() const override;
	EMediaStatus GetStatus() const override;
	TRangeSet<float> GetSupportedRates(EMediaRateThinning InThinning) const override;
	bool IsLooping() const override;
	bool Seek(const FTimespan& InTime) override;
	bool SetLooping(bool bInLooping) override;
	bool SetRate(float InRate) override;
	//~ End IMediaControls

	//~ Begin IMediaTracks
	bool GetAudioTrackFormat(int32 InTrackIndex, int32 InFormatIndex, FMediaAudioTrackFormat& OutFormat) const override;
	int32 GetNumTracks(EMediaTrackType InTrackType) const override;
	int32 GetNumTrackFormats(EMediaTrackType InTrackType, int32 InTrackIndex) const override;
	int32 GetSelectedTrack(EMediaTrackType InTrackType) const override;
	FText GetTrackDisplayName(EMediaTrackType InTrackType, int32 InTrackIndex) const override;
	int32 GetTrackFormat(EMediaTrackType InTrackType, int32 InTrackIndex) const override;
	FString GetTrackLanguage(EMediaTrackType InTrackType, int32 InTrackIndex) const override;
	FString GetTrackName(EMediaTrackType InTrackType, int32 InTrackIndex) const override;
	bool GetVideoTrackFormat(int32 InTrackIndex, int32 InFormatIndex, FMediaVideoTrackFormat& OutFormat) const override;
	bool SelectTrack(EMediaTrackType InTrackType, int32 InTrackIndex) override;
	bool SetTrackFormat(EMediaTrackType InTrackType, int32 InTrackIndex, int32 InFormatIndex) override;
	bool SetVideoTrackFrameRate(int32 InTrackIndex, int32 InFormatIndex, float InFrameRate) override;
	//~ End IMediaTracks

protected:

	void SetState(EMediaState InNewState);

	FString CurrentUrl;
	EMediaState CurrentState = EMediaState::Closed;
	EMediaStatus CurrentStatus = EMediaStatus::None;
	IMediaEventSink& EventSink;
	int32 NumAudioTracks = 0;
	int32 NumVideoTracks = 0;
	int32 SelectedVideoTrack = INDEX_NONE;
	int32 SelectedAudioTrack = INDEX_NONE;
};