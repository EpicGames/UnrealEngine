// Copyright Epic Games, Inc. All Rights Reserved.

#include "RtpCpuBufferWorker.h"

#include "RtpDecoderCpuBuffer.h"
#include "RtspMediaConstants.h"

#include "ElectraTextureSample.h"
#include "IElectraDecoderOutputVideo.h"
#include "MediaSamples.h"

#include "HAL/Event.h"
#include "HAL/PlatformProcess.h"

FRtpCpuBufferWorker::FRtpCpuBufferWorker()
	: bStopping(false)
{
	WaitEvent = FPlatformProcess::GetSynchEventFromPool();
}

FRtpCpuBufferWorker::~FRtpCpuBufferWorker()
{
	bStopping.store(true);

	if (WaitEvent)
	{
		WaitEvent->Trigger();
	}

	if (Thread.IsValid())
	{
		Thread->WaitForCompletion();
	}

	if (WaitEvent)
	{
		FPlatformProcess::ReturnSynchEventToPool(WaitEvent);
		WaitEvent = nullptr;
	}
}

bool FRtpCpuBufferWorker::Start(FMediaSamples* InSamples)
{
	if (!InSamples)
	{
		UE_LOGF(LogRtspMedia, Error, "CPU buffer worker requires a valid FMediaSamples");
		return false;
	}

	Samples = InSamples;
	bStopping.store(false);

	Thread.Reset(FRunnableThread::Create(this, TEXT("RtpCpuBufferWorker"), 0, TPri_Normal));
	if (!Thread.IsValid())
	{
		UE_LOGF(LogRtspMedia, Error, "Failed to create CPU buffer worker thread");
		return false;
	}

	return true;
}

void FRtpCpuBufferWorker::Enqueue(FCpuBufferWorkItem InItem)
{
	Queue.Enqueue(MoveTemp(InItem));
	if (WaitEvent)
	{
		WaitEvent->Trigger();
	}
}

uint32 FRtpCpuBufferWorker::Run()
{
	while (!bStopping.load())
	{
		if (WaitEvent && WaitEvent->Wait())
		{
			FCpuBufferWorkItem Item;
			while (!bStopping.load() && Queue.Dequeue(Item))
			{
				if (!Item.TextureSample.IsValid() || !Item.VideoOutput.IsValid())
				{
					continue;
				}

				RtspMedia::PopulateCpuBuffer(Item.TextureSample.Get(), Item.VideoOutput);

				if (!Item.TextureSample->Buffer.IsValid() && !bLoggedCpuBufferWarning)
				{
					UE_LOGF(LogRtspMedia, Warning, "bProvideCpuBuffer is enabled but the active decoder did not provide CPU-accessible data. "
						"GetBuffer() will return nullptr. Ensure the selected H.264 decoder supports the force_cpu_output codec option.");
					bLoggedCpuBufferWarning = true;
				}

				if (UE_LOG_ACTIVE(LogRtspMedia, VeryVerbose))
				{
					TOptional<FTimecode> OptionalTimecode = Item.TextureSample->GetTimecode();
					if (OptionalTimecode.IsSet())
					{
						const FTimecode Timecode = OptionalTimecode.GetValue();
						UE_LOGF(LogRtspMedia, VeryVerbose, "Adding video sample with timecode: %ls with subframe: %f", *Timecode.ToString(), Timecode.Subframe);
					}
					else
					{
						UE_LOGF(LogRtspMedia, VeryVerbose, "Adding video sample without timecode");
					}
				}

				Samples->AddVideo(Item.TextureSample.ToSharedRef());
			}
		}
	}

	return 0;
}

void FRtpCpuBufferWorker::Stop()
{
	bStopping.store(true);
	if (WaitEvent)
	{
		WaitEvent->Trigger();
	}
}
