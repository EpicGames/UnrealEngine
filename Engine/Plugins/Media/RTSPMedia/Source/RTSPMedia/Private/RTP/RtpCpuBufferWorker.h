// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/Queue.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "Templates/SharedPointer.h"

class FElectraTextureSample;
class FMediaSamples;
class IElectraDecoderVideoOutput;

// Work item handed from the decoder thread to the CPU-buffer worker. Holding
// the IElectraDecoderVideoOutput shared pointer is what keeps the underlying
// platform sample (e.g. IMFSample on Windows) alive past PollDecodedOutput.
struct FCpuBufferWorkItem
{
	TSharedPtr<FElectraTextureSample, ESPMode::ThreadSafe> TextureSample;
	TSharedPtr<IElectraDecoderVideoOutput, ESPMode::ThreadSafe> VideoOutput;
};

/**
 * Dedicated worker thread that performs the GPU->system staging copy
 * (RtspMedia::PopulateCpuBuffer) off the decoder thread and forwards the
 * resulting sample to FMediaSamples. On Windows MFT the staging copy is the
 * dominant per-frame cost (~8ms at 1080p NV12), so moving it here roughly
 * doubles end-to-end throughput.
 *
 * Single-producer / single-consumer: the owning FRtpDecoder's decoder thread
 * is the only producer; this thread is the only consumer. The owner must
 * stop the decoder thread before tearing this worker down so no enqueues
 * outlive the consumer.
 */
class FRtpCpuBufferWorker : public FRunnable
{
public:
	FRtpCpuBufferWorker();
	virtual ~FRtpCpuBufferWorker();

	// Starts the worker thread. Samples must outlive this worker.
	bool Start(FMediaSamples* InSamples);

	// Enqueued from the decoder thread (single producer). Items are processed
	// in order on the worker thread.
	void Enqueue(FCpuBufferWorkItem InItem);

private:
	// ~FRunnable Start
	virtual bool Init() override { return true; }
	virtual uint32 Run() override;
	virtual void Stop() override;
	// ~FRunnable Stop

	std::atomic<bool> bStopping;
	TUniquePtr<FRunnableThread> Thread;
	FEvent* WaitEvent = nullptr;
	TQueue<FCpuBufferWorkItem, EQueueMode::Spsc> Queue;
	FMediaSamples* Samples = nullptr;
	bool bLoggedCpuBufferWarning = false;
};
