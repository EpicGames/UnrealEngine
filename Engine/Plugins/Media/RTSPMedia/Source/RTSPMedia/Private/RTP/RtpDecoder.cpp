// Copyright Epic Games, Inc. All Rights Reserved.

#include "RtpDecoder.h"

#include "RtpH264Depacketizer.h"

#include "RtspMediaConstants.h"

#include "SDP/SdpSession.h"

// Electra
#include "ElectraTextureSample.h"
#include "IElectraDecoder.h"
#include "IElectraCodecFactory.h"
#include "IElectraCodecFactoryModule.h"
#include "IElectraDecoderOutputVideo.h"
#include "IElectraDecoderFeaturesAndOptions.h"
#include "MediaDecoderOutput.h"
#include "ParameterDictionary.h"
#include "Utils/MPEG/ElectraUtilsMPEGVideo_H264.h"
#include "Utils/MPEG/ElectraUtilsMPEGVideo.h"
#include "Utils/VideoDecoderHelpers.h"
#include "ElectraDecodersUtils.h"

#include "RtpDecoderCpuBuffer.h"
#include "RtpCpuBufferWorker.h"

#include "Async/Async.h"
#include "HAL/Event.h"
#include "HAL/PlatformProcess.h"
#include "Misc/ScopeExit.h"

FRtpDecoder::FRtpDecoder()
	: bStopping(false)
	, FrameRate(0.0)
{
	WaitEvent = FPlatformProcess::GetSynchEventFromPool();
}

FRtpDecoder::~FRtpDecoder()
{
	bStopping.store(true);

	if (WaitEvent)
	{
		WaitEvent->Trigger();
	}
	
	if (Thread.IsValid())
	{
		Thread->WaitForCompletion();
	}

	// Must wait for the thread to complete before we can return the event to the pool
	if (WaitEvent)
	{
		FPlatformProcess::ReturnSynchEventToPool(WaitEvent);
		WaitEvent = nullptr;
	}

	// Stop the CPU buffer worker only after the decoder thread (its sole
	// producer) has stopped, so no enqueues outlive the consumer.
	CpuBufferWorker.Reset();

	ResetPlatformVideoResource();
	TextureSamplePool.Reset();
	
	if (H264Depacketizer.IsValid())
	{
		H264Depacketizer->OnH264NalUnit.Unbind();
		H264Depacketizer->OnNalUnitSequenceGap.Unbind();
		H264Depacketizer.Reset();
	}

	if (ElectraDecoder.IsValid())
	{
		ElectraDecoder->Close();
	}
}

bool FRtpDecoder::Initialize(FMediaSamples* InSamples, const FSdpSession& InSession, int64 InMaxFragmentBufferSize, int64 InDecoderBufferSize, uint32 InDecoderPollIntervalMs, bool bInProvideCpuBuffer)
{
	Samples = InSamples;
	MaxFragmentBufferSize = InMaxFragmentBufferSize;
	DecoderBufferSize = InDecoderBufferSize;
	DecoderPollIntervalMs = InDecoderPollIntervalMs;
	bProvideCpuBuffer = bInProvideCpuBuffer;
	TextureSamplePool = MakeUnique<FElectraTextureSamplePool>();
	
	SdpSession = MakeUnique<FSdpSession>(InSession);
	
	if (!SdpSession.IsValid())
	{
		UE_LOGF(LogRtspMedia, Error, "Session unavailable when creating electra decoder");
		return false;
	}

	if (!SdpSession->HasVideo())
	{
		UE_LOGF(LogRtspMedia, Error, "Session must contain at least one video track");
		return false;
	}

	const FSdpMediaTrack* VideoTrack = SdpSession->GetFirstVideoTrack();
	if (!VideoTrack)
	{
		UE_LOGF(LogRtspMedia, Error, "Session did not provide a video track");
		return false;
	}

	// Store the clock rate to construct presentation time stamps when decoding
	const uint32 ClockRate = VideoTrack->ClockRate;
	if (ClockRate == 0)
	{
		UE_LOGF(LogRtspMedia, Error, "Session contains a zero clock rate when initializing decoder");
		return false;
	}
	VideoClockRate = ClockRate;

	// Seed SPS/PPS from SDP if available. Some encoders only provide parameter sets
	// out-of-band via sprop-parameter-sets and never send them in-band.
	// In-band updates will override these via the existing comparison in HandleOnH264NalUnit.
	const bool bHasSPS = !VideoTrack->SequenceParameterSet.IsEmpty();
	const bool bHasPPS = !VideoTrack->PictureParameterSet.IsEmpty();
	if (bHasSPS && bHasPPS)
	{
		SequenceParameterSet = VideoTrack->SequenceParameterSet;
		PictureParameterSet = VideoTrack->PictureParameterSet;
		bPendingCSDChange = true;
		UE_LOGF(LogRtspMedia, Verbose, "Seeded SPS (%d bytes) and PPS (%d bytes) from SDP",
			SequenceParameterSet.Num(), PictureParameterSet.Num());
	}
	else if (bHasSPS || bHasPPS)
	{
		UE_LOGF(LogRtspMedia, Warning, "SDP contains %ls but not %ls, deferring decoder creation until in-band parameter sets arrive",
			bHasSPS ? TEXT("SPS") : TEXT("PPS"),
			bHasSPS ? TEXT("PPS") : TEXT("SPS"));
	}
	else
	{
		UE_LOGF(LogRtspMedia, Verbose, "No SPS or PPS in SDP, deferring decoder creation until in-band parameter sets arrive");
	}

	return true;
}

bool FRtpDecoder::Start()
{
	// Start the CpuBufferWorker consumer before the decoder producer so 
	// any items the decoder thread enqueues immediately have a thread waiting to drain them.
	if (bProvideCpuBuffer)
	{
		CpuBufferWorker = MakeUnique<FRtpCpuBufferWorker>();
		if (!CpuBufferWorker->Start(Samples))
		{
			CpuBufferWorker.Reset();
			return false;
		}
	}

	bStopping.store(false);

	Thread.Reset(FRunnableThread::Create(this, TEXT("RtpDecoderThread"), 0, TPri_Normal));

	if (!Thread.IsValid())
	{
		UE_LOGF(LogRtspMedia, Error, "Failed to create decoder thread");
		CpuBufferWorker.Reset();
		return false;
	}

	return true;
}

void FRtpDecoder::Shutdown()
{
	Stop();
}

void FRtpDecoder::EnqueuePacket(FRtpPacket InPacket)
{
	Queue.Enqueue(FQueuedRtpPacket{MoveTemp(InPacket), FPlatformTime::Seconds()});
	if (WaitEvent)
	{
		WaitEvent->Trigger();
	}
}

double FRtpDecoder::GetFrameRate() const
{
	return FrameRate.load(std::memory_order_relaxed);
}

bool FRtpDecoder::Init()
{
	return true;
}

uint32 FRtpDecoder::Run()
{
	while (!bStopping.load())
	{
		// Check for any decoded output in the electra decoder
		PollDecodedOutput();

		// Delay until a packet arrives or the decoder poll interval expires
		if (WaitEvent && WaitEvent->Wait(DecoderPollIntervalMs))
		{
			FQueuedRtpPacket QueuedPacket;
			while (!bStopping.load() && Queue.Dequeue(QueuedPacket))
			{
				// If the decoder thread has fallen too far behind its input, drop frames
				// until the next IDR. P-frame reference dependencies rule out per-frame
				// dropping, so the next IDR is the earliest point we can resume cleanly.
				// The queue and depacketizer are reset so the recovery baseline comes from
				// the producer's next packet without stale-stamp re-triggers or sequence
				// gap warnings.
				if (!bWaitForIDRFrame)
				{
					const double LagSeconds = FPlatformTime::Seconds() - QueuedPacket.EnqueueTimeSeconds;
					const int32 LagMs = static_cast<int32>(LagSeconds * 1000.0);
					if (LagMs > RtspMedia::Default::DecoderLagThresholdMs)
					{
						UE_LOGF(LogRtspMedia, Warning, "RtpDecoder fell %dms behind input. Dropping frames until next IDR.", LagMs);

						Queue.Empty();

						// Tear down the depacketizer so the next packet seeds a fresh sequence
						// number baseline. Without this we would immediately re-enter recovery
						// via OnNalUnitSequenceGap against the packets we just dropped.
						if (H264Depacketizer.IsValid())
						{
							H264Depacketizer->OnH264NalUnit.Unbind();
							H264Depacketizer->OnNalUnitSequenceGap.Unbind();
							H264Depacketizer.Reset();
						}

						BeginIDRRecovery();

						// Skip the trigger packet so the new depacketizer's sequence baseline
						// comes from the producer's next packet, not from one with a known gap.
						continue;
					}
				}

				if (!H264Depacketizer.IsValid())
				{
					H264Depacketizer = MakeUnique<FRtpH264Depacketizer>();
					H264Depacketizer->SetMaxFragmentBufferSize(MaxFragmentBufferSize);
					H264Depacketizer->OnH264NalUnit.BindRaw(this, &FRtpDecoder::HandleOnH264NalUnit);
					H264Depacketizer->OnNalUnitSequenceGap.BindRaw(this, &FRtpDecoder::HandleOnNalUnitSequenceGap);
				}
				const FRtpHeader Header = QueuedPacket.Packet.Header;
				H264Depacketizer->ProcessRtpPayload(MoveTemp(QueuedPacket.Packet.Payload), Header.SequenceNumber, Header.Timestamp, Header.bMarker);
			}
		}
	}

	return 0;
}

void FRtpDecoder::Stop()
{
	bStopping.store(true);
	if (WaitEvent)
	{
		WaitEvent->Trigger();
	}
}

 bool FRtpDecoder::CreateElectraDecoder(const TArray<uint8>& InSequenceParameterSet, const TArray<uint8>& InPictureParameterSet)
 {
	// Build Codec Specific Data ('csd')
	// Used to initialize the decoder and for each decode action.
	// Contains the sequence and picture parameter sets which include information like:
	// SPS - Properties for the entire sequence of frames
	// E.g. Resolution, profile, chroma format, bit depth
	// PPS - Per-frame properties
	// E.g. Coding mode, quantization parameters

	// H.264 Annex B start code is required before the SPS and PPS
	constexpr uint8 H264AnnexBStartCode[] = {0x00, 0x00, 0x00, 0x01};

	TArray<uint8> CodecSpecificData;
	CodecSpecificData.Append(H264AnnexBStartCode, 4);
	CodecSpecificData.Append(InSequenceParameterSet);
	CodecSpecificData.Append(H264AnnexBStartCode, 4);
	CodecSpecificData.Append(InPictureParameterSet);

	CodecOptions.Empty();
	CodecTypeFormat = {};

	// Convert Annex B style codec specific data to an avcC configuration record
	ElectraDecodersUtil::MPEG::H264::FAVCDecoderConfigurationRecord AvcConfigurationRecord;
	if (AvcConfigurationRecord.CreateFromCodecSpecificData(CodecSpecificData))
	{
		// DCR = Decoder Configuration Record
		CodecOptions.Emplace("dcr", AvcConfigurationRecord.GetRawData());
		CodecOptions.Emplace("csd", AvcConfigurationRecord.GetCodecSpecificData());
		CodecOptions.Emplace("max_output_buffers", FVariant(DecoderBufferSize));
		if (bProvideCpuBuffer)
		{
			CodecOptions.Emplace("force_cpu_output", FVariant(static_cast<int64>(1)));
		}

		// 'avc1' is a flavour of H.264 where the Sequence and Picture parameter sets are stored 'out of band'
		// This means they are stored separately from the main H.264 bitstream, which is the case when streaming with RTSP.
		// In RTSP these parameter sets are provided within the SDP (Session Description Protocol) data when the connection handshake occurs.
		CodecTypeFormat.RFC6381 = AvcConfigurationRecord.GetCodecSpecifierRFC6381(TEXT("avc1"));
		CodecTypeFormat.DCR = AvcConfigurationRecord.GetRawData();
		if (!ElectraDecodersUtil::PrepareCodecTypeFormat(CodecTypeFormat) || !CodecTypeFormat.Properties.IsType<Electra::FCodecTypeFormat::FVideo>())
		{
			UE_LOGF(LogRtspMedia, Error, "Failed to prepare codec type format from H.264 configuration");
			return false;
		}
		const Electra::FCodecTypeFormat::FVideo& vid(CodecTypeFormat.Properties.Get<Electra::FCodecTypeFormat::FVideo>());
		if (vid.Width && vid.Height)
		{
			CodecOptions.Emplace("max_width", FVariant(static_cast<int64>(vid.Width)));
			CodecOptions.Emplace("max_height", FVariant(static_cast<int64>(vid.Height)));
			UE_LOGF(LogRtspMedia, Verbose, "Determined max image dimensions for decoder from SPS. MaxWidth: %d MaxHeight %d", vid.Width, vid.Height);
		}
		else
		{
			UE_LOGF(LogRtspMedia, Error, "No or invalid sequence parameter sets within H.264 configuration");
			return false;
		}
	}
	else
	{
		UE_LOGF(LogRtspMedia, Error, "Failed to create AVC configuration record from CSD");
		return false;
	}

	IElectraCodecFactoryModule* FactoryModule = FModuleManager::Get().GetModulePtr<IElectraCodecFactoryModule>("ElectraCodecFactory");
	if (!FactoryModule)
	{
		UE_LOGF(LogRtspMedia, Error, "Failed to get ElectraCodecFactoryModule");
		return false;
	}

	// Codec format is: avc1.XXYYZZ
	// Using 'avc1' here hard codes us to H.264 which is fine whilst this is the only protocol that we support.
	// XX is profile_idc (Profile Indicator - Describes the encoder feature set e.g 'Baseline', 'Main' or 'High')
	// YY is constraint flags (Indicates constraints the encoder followed, e.g. a Main profile encoder only used Baseline features)
	// ZZ is level_idc (Level Indicator - Indicates the resources required for decoding. e.g. 31, Level 3.1 indicates 1280x720 @ 30 FPS)
	//
	// E.g. "avc1.42001F"
	// 42 -> 66 -> Baseline profile
	// 00 -> No constraint flags set
	// 1F -> Level 31 -> Level 3.1 (1280 x 720 @ 30 FPS)
	//
	// To get this information the first four bytes of the SPS contain:
	// 0: NAL Unit Header
	// 1: profile_idc
	// 2: constraint flags
	// 3: level_idc

	if (CodecTypeFormat.Type == Electra::FCodecTypeFormat::EType::Invalid)
	{
		UE_LOGF(LogRtspMedia, Error, "CodecFormat was empty when preparing to create decoder");
		return false;
	}
	
	TMap<FString, FVariant> FormatInfo;
	CodecFactory = FactoryModule->GetBestDecoderFactoryForFormat(FormatInfo, CodecTypeFormat, CodecOptions);
	if (CodecFactory)
	{
		if (ElectraDecoder.IsValid())
		{
			ElectraDecoder->Close();
		}

		ElectraDecoder = CodecFactory->CreateDecoder(CodecTypeFormat, CodecOptions);

		if (!ElectraDecoder.IsValid())
		{
			UE_LOGF(LogRtspMedia, Error, "Failed to create decoder for format: %ls", *CodecTypeFormat.RFC6381);
			return false;
		}
	}
	else
	{
		UE_LOGF(LogRtspMedia, Error, "Failed to create codec factory for unsupported format: %ls", *CodecTypeFormat.RFC6381);
		return false;
	}

	BitstreamProcessor.Reset();
	BitstreamInfoQueue.Empty();
	BitstreamProcessor = ElectraDecoder->CreateBitstreamProcessor();

	if (!BitstreamProcessor.IsValid())
	{
		UE_LOGF(LogRtspMedia, Error, "Decoder failed to create bitstream processor");
		return false;
	}

	return true;
 }

void FRtpDecoder::HandleOnNalUnitSequenceGap()
{
	// A sequence gap means the access unit currently being assembled can never
	// complete correctly. Recover by dropping partial state and waiting for the
	// next IDR. The depacketizer has already reset its FU-A reassembly.
	BeginIDRRecovery();
}

void FRtpDecoder::ResetIDRRecoveryState()
{
	bWaitForIDRFrame = true;

	FrameBuffer.Empty();
	BitstreamInfoQueue.Empty();

	LastPresentationTimestamp = FTimespan::Zero();
	NumFramesSinceLastUpdate = 0;
	LastFrameRateUpdate = 0.0;
}

void FRtpDecoder::BeginIDRRecovery()
{
	ResetIDRRecoveryState();

	// Drop the decoder's internal reference state so the next IDR is decoded
	// cleanly. Without this the MFT can emit corrupted or extra concealed
	// frames against stale references, which then desync the bitstream-info
	// pairing and produce repeated "no pending decoder input" errors.
	//
	// A failed flush leaves the decoder in an unusable error state per the
	// IElectraDecoder contract, so recreate it instead of limping on.
	if (ElectraDecoder.IsValid() && ElectraDecoder->Flush() == IElectraDecoder::EDecoderError::Error)
	{
		const IElectraDecoder::FError FlushError = ElectraDecoder->GetError();
		UE_LOGF(LogRtspMedia, Warning, "ElectraDecoder flush failed during IDR recovery: \"%ls\" Code: %d SDK Error: %d. Recreating.", *FlushError.GetMessage(), FlushError.GetCode(), FlushError.GetSdkCode());
		TryRecreateDecoder();
	}
}

bool FRtpDecoder::TryRecreateDecoder()
{
	if (CreateElectraDecoder(SequenceParameterSet, PictureParameterSet))
	{
		return true;
	}

	UE_LOGF(LogRtspMedia, Error, "Failed to recreate ElectraDecoder.");
	OnDecoderError.ExecuteIfBound();
	bStopping.store(true);
	return false;
}

void FRtpDecoder::HandleOnH264NalUnit(FRtpH264NalUnit InNalUnit)
{
	// Track in-band SPS/PPS for mid-stream resolution changes (e.g. iPad orientation change).
	if (!InNalUnit.SequenceParameterSet.IsEmpty() && InNalUnit.SequenceParameterSet != SequenceParameterSet)
	{
		UE_LOGF(LogRtspMedia, Verbose, "In-band SPS changed (%d bytes)", InNalUnit.SequenceParameterSet.Num());
		SequenceParameterSet = MoveTemp(InNalUnit.SequenceParameterSet);
		bPendingCSDChange = true;
	}

	if (!InNalUnit.PictureParameterSet.IsEmpty() && InNalUnit.PictureParameterSet != PictureParameterSet)
	{
		UE_LOGF(LogRtspMedia, Verbose, "In-band PPS changed (%d bytes)", InNalUnit.PictureParameterSet.Num());
		PictureParameterSet = MoveTemp(InNalUnit.PictureParameterSet);
		bPendingCSDChange = true;
	}

	// Wait on an IDR frame first then allow all other units through.
	if (bWaitForIDRFrame)
	{
		if (!InNalUnit.bIsIDRFrame)
		{
			// Abort processing until we get one
			return;
		}
		bWaitForIDRFrame = false;

		// Ensure we're starting with a fresh frame buffer for the IDR frame
		ResetFrameBuffer();
	}

	// Append the NAL unit to our frame buffer
	FrameBuffer.Append(InNalUnit.Data);

	// Marker may indicate a complete access unit.
	// However, we can't use it to reliably start decoding.
	// https://www.rfc-editor.org/rfc/rfc6184#section-5.1
	// See the bIsSlice check below.
	if (!InNalUnit.bIsMarkerBitSet)
	{
		return;
	}

	// Some senders will set the marker bit on NAL units that contain only SPS/PPS data (metadata, no image data).
	// The decoder requires a complete access unit with an image slice or decoding will fail.
	// So once the marker bit is set we need to check that we have received a NAL unit that represents an image slice.
	// Any metadata units we receive will be aggregated into the next access unit.
	// With the marker bit set and bIsSlice == true, we should have a complete unit containing image data ready for decoding.
	if (!InNalUnit.bIsSlice)
	{
		return;
	}

	bool bDecodedSuccessfully = false;

	// After this point we should have a complete frame in the buffer
	// Successful decoding or failure should reset the frame buffer from here on.
	ON_SCOPE_EXIT
	{
		ResetFrameBuffer();

		// We always set bWaitForIDRFrame to true on return EXCEPT for when we decoded successfully.
		// In the success case we want to continue processing non IDR-frames
		if (!bDecodedSuccessfully)
		{
			bWaitForIDRFrame = true;
		}
	};
	
	// An access unit represents one frame of data, consisting of multiple NAL units.
	IElectraDecoder::FInputAccessUnit AccessUnit;
	AccessUnit.Data = FrameBuffer.GetData();
	AccessUnit.DataSize = FrameBuffer.Num();
	AccessUnit.PTS = FTimespan::FromSeconds(static_cast<double>(InNalUnit.Timestamp) / VideoClockRate);
	if (InNalUnit.bIsIDRFrame)
	{
		AccessUnit.Flags = EElectraDecoderFlags::IsSyncSample;
	}

	// Create or recreate the decoder when SPS/PPS has changed.
	// On the first frame this creates the decoder from in-band SPS/PPS (deferred from Initialize).
	// On subsequent changes (e.g. iPad orientation) it recreates with the new parameters.
	if (bPendingCSDChange)
	{
		if (SequenceParameterSet.IsEmpty() || PictureParameterSet.IsEmpty())
		{
			UE_LOGF(LogRtspMedia, Verbose, "Waiting for both SPS and PPS before creating decoder");
			return;
		}

		UE_LOGF(LogRtspMedia, Verbose, "CSD changed, %ls decoder", ElectraDecoder.IsValid() ? TEXT("recreating") : TEXT("creating"));
		bPendingCSDChange = false;

		if (!CreateElectraDecoder(SequenceParameterSet, PictureParameterSet))
		{
			UE_LOGF(LogRtspMedia, Error, "Failed to create decoder from in-band SPS/PPS");
			OnDecoderError.ExecuteIfBound();
			bStopping.store(true);
			return;
		}
	}

	if (!BitstreamProcessor.IsValid() || !ElectraDecoder.IsValid())
	{
		UE_LOGF(LogRtspMedia, Warning, "Decoder not yet available, waiting for in-band SPS/PPS");
		return;
	}

	// Before decoding can take place, any platform specific preparation needs to be performed.
	// On some platforms the big endian 4 byte length indicator we output in the depacketizer will be swapped
	// for a H264 Annex B start code.
	// If we were using ElectraPlayer directly then we wouldn't need to perform this step.
	// However, as we're directly integrating with the underlying decoders we need to prepare the data for processing.
	TSharedPtr<IElectraDecoderBitstreamInfo, ESPMode::ThreadSafe> BitstreamInfo;
	const IElectraDecoderBitstreamProcessor::EProcessResult ProcessResult = BitstreamProcessor->ProcessInputForDecoding(BitstreamInfo, AccessUnit, CodecOptions);
	if (ProcessResult == IElectraDecoderBitstreamProcessor::EProcessResult::Error)
	{
		UE_LOGF(LogRtspMedia, Warning, "BitstreamProcessor error: %ls", *BitstreamProcessor->GetLastError());
		return;
	}

	const IElectraDecoder::EDecoderError DecodeResult = ElectraDecoder->DecodeAccessUnit(AccessUnit, CodecOptions);

	switch (DecodeResult)
	{
		case IElectraDecoder::EDecoderError::None:
			bDecodedSuccessfully = true;
			BitstreamInfoQueue.Enqueue(BitstreamInfo);
			break;
		case IElectraDecoder::EDecoderError::NoBuffer:
			UE_LOGF(LogRtspMedia, Verbose, "ElectraDecoder reports no buffer available, dropping unit");
			return;
		case IElectraDecoder::EDecoderError::LostDecoder:
			UE_LOGF(LogRtspMedia, Warning, "Lost decoder reported. Recreating.");
			ResetIDRRecoveryState();
			TryRecreateDecoder();
			return;
		default:
			IElectraDecoder::FError DecodeError = ElectraDecoder->GetError();
			UE_LOGF(LogRtspMedia, Warning, "ElectraDecoder error: \"%ls\" Code: %d SDK Error: %d", *DecodeError.GetMessage(), DecodeError.GetCode(), DecodeError.GetSdkCode());
			break;
	}

	// Most frames will be picked up in the Run loop, but this will catch any frames that are decoded quickly.
	PollDecodedOutput();
}

void FRtpDecoder::PollDecodedOutput()
{
	if (!ElectraDecoder.IsValid())
	{
		UE_LOGF(LogRtspMedia, VeryVerbose, "Attempted to process decoded output without an electra decoder instance");
		return;
	}

	if (!TextureSamplePool.IsValid())
	{
		UE_LOGF(LogRtspMedia, Error, "Texture sample buffer not available when processing decoded output");
		return;
	}

	while (ElectraDecoder->HaveOutput() == IElectraDecoder::EOutputStatus::Available)
	{
		TSharedPtr<IElectraDecoderOutput> Output = ElectraDecoder->GetOutput();

		// Dequeue the corresponding bitstream info for this output frame.
		// This must happen before any continue/discard paths to keep the queue in sync.
		TSharedPtr<IElectraDecoderBitstreamInfo, ESPMode::ThreadSafe> FrameBitstreamInfo;
		BitstreamInfoQueue.Dequeue(FrameBitstreamInfo);

		if (!Output.IsValid())
		{
			UE_LOGF(LogRtspMedia, Warning, "Received null output from electra decoder");
			continue;
		}

		TSharedPtr<IElectraDecoderVideoOutput> VideoOutput = StaticCastSharedPtr<IElectraDecoderVideoOutput>(Output);
		
		if (!VideoOutput.IsValid())
		{
			UE_LOGF(LogRtspMedia, Warning, "Cast from IElectraDecoderOutput to IElectraDecoderVideoOutput failed");
			continue;
		}

		if (VideoOutput->GetOutputType() == IElectraDecoderVideoOutput::EOutputType::DoNotOutput)
		{
			// There is a problem with the output, discard it.
			UE_LOGF(LogRtspMedia, Warning, "Video output type is 'DoNotOutput'. Discarding");
			continue;
		}
		
		TSharedPtr<FElectraTextureSample, ESPMode::ThreadSafe> TextureSample = TextureSamplePool->AcquireShared();
		if (!TextureSample.IsValid())
		{
			UE_LOGF(LogRtspMedia, Error, "Failed to acquire texture sample from pool");
			continue;
		}

		// Get video dimensions for:
		// - Providing format information to the Media framework
		// - Populating the platform video resource options below
		const int32 VideoWidth = VideoOutput->GetWidth();
		const int32 VideoHeight = VideoOutput->GetHeight();
		
		// If the video dimensions change reset any platform video resources we may be using,
		// updated the cached dimensions and inform the delegate on the game thread.
		if (VideoWidth != CurrentVideoWidth || VideoHeight != CurrentVideoHeight)
		{
			ResetPlatformVideoResource();
			CurrentVideoWidth = VideoWidth;
			CurrentVideoHeight = VideoHeight;
			if (OnVideoDimensions.IsBound())
			{
				OnVideoDimensions.Execute(VideoWidth, VideoHeight);
			}
		}

		// For some platforms we must specify the required resources for the texture
		// On Apple platforms this information is ignored, but it is used on Windows and Linux
		if (!PlatformResource)
		{
			TMap<FString, FVariant> PlatformOptions;
			PlatformOptions.Emplace(TEXT("max_width"), FVariant(static_cast<int64>(VideoWidth)));
			PlatformOptions.Emplace(TEXT("max_height"), FVariant(static_cast<int64>(VideoHeight)));
			PlatformResource = FElectraDecodersPlatformResources::CreatePlatformVideoResource(this, PlatformOptions);
		}

		// Calculate frame duration from presentation timestamps
		const FTimespan CurrentPresentationTimestamp = VideoOutput->GetPTS();
		if (!LastPresentationTimestamp.IsZero() && CurrentPresentationTimestamp > LastPresentationTimestamp)
		{
			const FTimespan FrameDuration = CurrentPresentationTimestamp - LastPresentationTimestamp;
			FrameDurationTicks = FrameDuration.GetTicks();

			NumFramesSinceLastUpdate++;
			
			// Update frame rate
			constexpr double FrameRateUpdateIntervalSeconds = 1.0;
			const double Now = FPlatformTime::Seconds();
			const double TimeSinceLastUpdate = Now - LastFrameRateUpdate;
			if (TimeSinceLastUpdate > FrameRateUpdateIntervalSeconds)
			{
				const double Average = TimeSinceLastUpdate / NumFramesSinceLastUpdate;
				const double NewFrameRate = 1.0 / Average;
				FrameRate.store(FMath::RoundToDouble(NewFrameRate), std::memory_order_relaxed);
				NumFramesSinceLastUpdate = 0;
				LastFrameRateUpdate = Now;
			}
		}
		LastPresentationTimestamp = CurrentPresentationTimestamp;

		// Populate buffer timing properties 
		TSharedPtr<Electra::FParamDict, ESPMode::ThreadSafe> BufferProperties = MakeShared<Electra::FParamDict, ESPMode::ThreadSafe>();

		// Electra uses its own time values that we need to map
		// One FTimespan tick is 100 nanoseconds
		const int64 PresentationTimeTicks = CurrentPresentationTimestamp.GetTicks();
		BufferProperties->Set(IDecoderOutputOptionNames::PTS, Electra::FVariantValue(Electra::FTimeValue(PresentationTimeTicks)));
		BufferProperties->Set(IDecoderOutputOptionNames::Duration, Electra::FVariantValue(Electra::FTimeValue(FrameDurationTicks)));

		// Extract timecode from bitstream processor SEI data
		if (BitstreamProcessor.IsValid() && FrameBitstreamInfo.IsValid())
		{
			TMap<FString, FVariant> BsiProperties;
			BitstreamProcessor->SetPropertiesOnOutput(BsiProperties, FrameBitstreamInfo);

			TArray<uint8> PicTiming = ElectraDecodersUtil::GetVariantValueUInt8Array(BsiProperties, IElectraDecoderBitstreamProcessorInfo::CommonPictureTiming);
			if (PicTiming.Num() == sizeof(ElectraDecodersUtil::MPEG::FCommonPictureTiming))
			{
				TSharedPtr<Electra::MPEG::FVideoDecoderTimecode> Timecode = MakeShared<Electra::MPEG::FVideoDecoderTimecode, ESPMode::ThreadSafe>();
				Timecode->UpdateWith(*reinterpret_cast<const ElectraDecodersUtil::MPEG::FCommonPictureTiming*>(PicTiming.GetData()));
				BufferProperties->Set(IDecoderOutputOptionNames::Timecode, Electra::FVariantValue(Timecode));
			}
		}

		FString SetupOutputTextureSampleError;
		const bool bSuccess = FElectraDecodersPlatformResources::SetupOutputTextureSample(
			SetupOutputTextureSampleError, 
			TextureSample, 
			VideoOutput, 
			BufferProperties, 
			PlatformResource
		);

		if (!bSuccess)
		{
			UE_LOGF(LogRtspMedia, Warning, "Failed to set up texture sample: %ls", *SetupOutputTextureSampleError);
			continue;
		}

		// When CPU-buffer output is requested, hand the sample off to the worker
		// thread for the GPU->system staging copy. The worker forwards the sample
		// to FMediaSamples after staging completes. The VideoOutput reference
		// retained in the work item keeps the underlying platform sample alive
		// past this PollDecodedOutput iteration.
		if (CpuBufferWorker.IsValid())
		{
			FCpuBufferWorkItem WorkItem;
			WorkItem.TextureSample = TextureSample;
			WorkItem.VideoOutput = VideoOutput;
			CpuBufferWorker->Enqueue(MoveTemp(WorkItem));
			continue;
		}

		if (UE_LOG_ACTIVE(LogRtspMedia, VeryVerbose))
		{
			TOptional<FTimecode> OptionalTimecode = TextureSample->GetTimecode();
			if (OptionalTimecode.IsSet())
			{
				const FTimecode Timecode = OptionalTimecode.GetValue();
				UE_LOGF(LogRtspMedia, VeryVerbose, "Adding video sample with timecode: %ls with subframe: %f", *Timecode.ToString(), Timecode.Subframe);
			}
			else
			{
				UE_LOGF(LogRtspMedia, VeryVerbose, "Adding video sample without timecode");
			}
		}

		Samples->AddVideo(TextureSample.ToSharedRef());
	}
}

void FRtpDecoder::ResetFrameBuffer()
{
	FrameBuffer.Empty();
}

void FRtpDecoder::ResetPlatformVideoResource()
{
	if (PlatformResource)
	{
		FElectraDecodersPlatformResources::ReleasePlatformVideoResource(this, PlatformResource);
		PlatformResource = nullptr;
	}
}
