// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RtpH264NalUnit.h"

#include "RtspMediaDefaults.h"

#include "RtpPacket.h"

#include "Containers/Queue.h"
#include "CoreMinimal.h"
#include "ElectraDecodersPlatformResources.h"
#include "CodecTypeFormat.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "MediaSamples.h"
#include "Misc/Variant.h"
#include "Templates/SharedPointer.h"

class IElectraDecoder;
class IElectraDecoderBitstreamInfo;
class IElectraDecoderBitstreamProcessor;
class IElectraCodecFactory;
class IElectraDecoderVideoOutput;
class FElectraTextureSample;
class FElectraTextureSamplePool;
class FMediaSamples;
class FRtpH264Depacketizer;
class FRtpCpuBufferWorker;

struct FSdpSession;

struct FQueuedRtpPacket
{
	FRtpPacket Packet;
	double EnqueueTimeSeconds = 0.0;
};

// Provides Width and Height, executed on the decoder thread.
DECLARE_DELEGATE_TwoParams(FOnVideoDimensions, int32, int32);

DECLARE_DELEGATE(FOnDecoderError);

/**
 * Uses a dedicated worker thread for:
 * - Depacketizing H.264 NAL units
 * - H.264 Decoder execution
 *
 * The Initialize, Start and Shutdown methods are all expected to be called from the game thread.
 * EnqueuePacket is expected to be called from the RtspClient worker thread to avoid
 * a hop to the game thread in order to minimise latency.
 */
class FRtpDecoder : public FRunnable
{
public:
	FRtpDecoder();
	virtual ~FRtpDecoder();
	
	bool Initialize(
		FMediaSamples* InSamples,
		const FSdpSession& InSession,
		int64 InMaxFragmentBufferSize,
		int64 InDecoderBufferSize,
		uint32 InDecoderPollIntervalMs,
		bool bInProvideCpuBuffer);
	
	bool Start();
	void Shutdown();
	void EnqueuePacket(FRtpPacket InPacket);

	double GetFrameRate() const;

	FOnVideoDimensions OnVideoDimensions;
	FOnDecoderError OnDecoderError;

private:
	// ~FRunnable Start
	virtual bool Init() override;
	virtual uint32 Run() override;
	virtual void Stop() override;
	// ~FRunnable Stop

	bool CreateElectraDecoder(const TArray<uint8>& InSequenceParameterSet, const TArray<uint8>& InPictureParameterSet);
	void HandleOnH264NalUnit(FRtpH264NalUnit InNalUnit);
	void HandleOnNalUnitSequenceGap();
	void PollDecodedOutput();

	// Clears the partial access unit, pending bitstream info, and frame-timing
	// state, and sets bWaitForIDRFrame. Does not touch the ElectraDecoder.
	void ResetIDRRecoveryState();

	// Resets recovery state and flushes the ElectraDecoder so the next IDR is
	// decoded cleanly. Recreates the decoder if the flush itself fails.
	void BeginIDRRecovery();

	// Tears down the existing ElectraDecoder and recreates it from the cached
	// SPS/PPS. On failure, signals OnDecoderError and sets bStopping. Returns
	// true on success.
	bool TryRecreateDecoder();

	void ResetFrameBuffer();
	void ResetPlatformVideoResource();

	TQueue<FQueuedRtpPacket, EQueueMode::Spsc> Queue;

	// Thread Management
	std::atomic<bool> bStopping;
	TUniquePtr<FRunnableThread> Thread;
	FEvent* WaitEvent;

	// Decoding Tools
	TSharedPtr<IElectraDecoderBitstreamProcessor, ESPMode::ThreadSafe> BitstreamProcessor;
	TSharedPtr<IElectraCodecFactory> CodecFactory;
	TSharedPtr<IElectraDecoder> ElectraDecoder;
	TUniquePtr<FRtpH264Depacketizer> H264Depacketizer;

	// Decoding Configuration
	TArray<uint8> SequenceParameterSet;
	TArray<uint8> PictureParameterSet;
	Electra::FCodecTypeFormat CodecTypeFormat;
	TMap<FString, FVariant> CodecOptions;
	int64 MaxFragmentBufferSize = RtspMedia::Default::MaxFragmentBufferSizeBytes;
	int64 DecoderBufferSize = RtspMedia::Default::DecoderBufferSize;
	uint32 DecoderPollIntervalMs = RtspMedia::Default::DecoderPollIntervalMs;
	bool bProvideCpuBuffer = RtspMedia::Default::bProvideCpuBuffer;

	// Off-thread CPU-buffer staging. Created in Initialize when bProvideCpuBuffer
	// is true. The decoder thread enqueues TextureSample/VideoOutput pairs; the
	// worker performs the GPU->system staging copy (~8ms/frame at 1080p NV12 on
	// Windows MFT) and forwards the sample to FMediaSamples.
	TUniquePtr<FRtpCpuBufferWorker> CpuBufferWorker;

	// Decoding State
	TArray<uint8> FrameBuffer;
	TUniquePtr<FSdpSession> SdpSession;
	TQueue<TSharedPtr<IElectraDecoderBitstreamInfo, ESPMode::ThreadSafe>> BitstreamInfoQueue;
	uint32 VideoClockRate = 90000;
	bool bWaitForIDRFrame = true;
	bool bPendingCSDChange = false;

	// Texture Sample Generation
	int32 CurrentVideoWidth = 0;
	int32 CurrentVideoHeight = 0;
	FTimespan LastPresentationTimestamp;
	FElectraDecodersPlatformResources::IDecoderPlatformResource* PlatformResource = nullptr;
	TUniquePtr<FElectraTextureSamplePool> TextureSamplePool;
	int64 FrameDurationTicks = 333333;
	FMediaSamples* Samples = nullptr;
	
	// Frame rate will be updated on the decoder thread and read by the game thread for UI display
	std::atomic<double> FrameRate;
	double LastFrameRateUpdate = 0.0;
	int32 NumFramesSinceLastUpdate = 0;
};
