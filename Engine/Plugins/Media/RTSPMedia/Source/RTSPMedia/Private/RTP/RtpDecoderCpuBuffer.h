// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class FElectraTextureSample;
class IElectraDecoderVideoOutput;

namespace RtspMedia
{
	/**
	 * Populates FElectraTextureSample::Buffer with NV12 pixel data for CPU access.
	 * Called after SetupOutputTextureSample succeeds. The GPU rendering path is unaffected.
	 *
	 * Platform behavior:
	 * - macOS/iOS: Locks the CVPixelBuffer and copies Y + CbCr planes
	 * - Windows: Extracts MFSample via GetPlatformOutputHandle and copies via Lock2D
	 * - Linux: No-op (SetupOutputTextureSample already populates Buffer)
	 */
	void PopulateCpuBuffer(FElectraTextureSample* InTextureSample, const TSharedPtr<IElectraDecoderVideoOutput>& InVideoOutput);
}
