// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/Platform.h"

#if PLATFORM_APPLE

#include "RtpDecoderCpuBuffer.h"

#include "RtspMediaConstants.h"

#include "ElectraTextureSample.h"

#include "Misc/ScopeExit.h"

#include <CoreVideo/CoreVideo.h>

void RtspMedia::PopulateCpuBuffer(FElectraTextureSample* InTextureSample, const TSharedPtr<IElectraDecoderVideoOutput>& InVideoOutput)
{
	if (!InTextureSample || !InVideoOutput.IsValid())
	{
		return;
	}

	// On Apple platforms, SetupOutputTextureSample populates ImageBufferRef for GPU rendering.
	// Lock the CVPixelBuffer to copy its contents into Buffer for CPU access.
	CVPixelBufferRef PixelBuffer = InTextureSample->ImageBufferRef;
	if (!PixelBuffer)
	{
		UE_LOGF(LogRtspMedia, Warning, "PopulateCpuBuffer: No CVPixelBuffer available");
		return;
	}

	const CVReturn LockResult = CVPixelBufferLockBaseAddress(PixelBuffer, kCVPixelBufferLock_ReadOnly);
	if (LockResult != kCVReturnSuccess)
	{
		UE_LOGF(LogRtspMedia, Warning, "PopulateCpuBuffer: CVPixelBufferLockBaseAddress failed (%d)", static_cast<int32>(LockResult));
		return;
	}

	ON_SCOPE_EXIT
	{
		CVPixelBufferUnlockBaseAddress(PixelBuffer, kCVPixelBufferLock_ReadOnly);
	};

	const size_t PlaneCount = CVPixelBufferGetPlaneCount(PixelBuffer);
	if (PlaneCount < 2)
	{
		UE_LOGF(LogRtspMedia, Warning, "PopulateCpuBuffer: Expected 2 planes (NV12), got %d", static_cast<int32>(PlaneCount));
		return;
	}

	// Y plane
	const uint8* YBaseAddress = static_cast<const uint8*>(CVPixelBufferGetBaseAddressOfPlane(PixelBuffer, 0));
	const size_t YBytesPerRow = CVPixelBufferGetBytesPerRowOfPlane(PixelBuffer, 0);
	const size_t YHeight = CVPixelBufferGetHeightOfPlane(PixelBuffer, 0);

	// CbCr plane (interleaved UV)
	const uint8* CbCrBaseAddress = static_cast<const uint8*>(CVPixelBufferGetBaseAddressOfPlane(PixelBuffer, 1));
	const size_t CbCrBytesPerRow = CVPixelBufferGetBytesPerRowOfPlane(PixelBuffer, 1);
	const size_t CbCrHeight = CVPixelBufferGetHeightOfPlane(PixelBuffer, 1);

	if (!YBaseAddress || !CbCrBaseAddress)
	{
		UE_LOGF(LogRtspMedia, Warning, "PopulateCpuBuffer: CVPixelBufferGetBaseAddressOfPlane returned null (Y=%p, CbCr=%p)", YBaseAddress, CbCrBaseAddress);
		return;
	}

	const size_t TotalBytes = (YBytesPerRow * YHeight) + (CbCrBytesPerRow * CbCrHeight);

	InTextureSample->Buffer = MakeShared<TArray64<uint8>, ESPMode::ThreadSafe>();
	InTextureSample->Buffer->SetNumUninitialized(static_cast<int64>(TotalBytes));

	uint8* Dest = InTextureSample->Buffer->GetData();
	FMemory::Memcpy(Dest, YBaseAddress, YBytesPerRow * YHeight);
	FMemory::Memcpy(Dest + (YBytesPerRow * YHeight), CbCrBaseAddress, CbCrBytesPerRow * CbCrHeight);

	InTextureSample->Stride = static_cast<uint32>(YBytesPerRow);
}

#endif // PLATFORM_APPLE
