// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/Platform.h"

#if PLATFORM_LINUX

#include "RtpDecoderCpuBuffer.h"

void RtspMedia::PopulateCpuBuffer(FElectraTextureSample* InTextureSample, const TSharedPtr<IElectraDecoderVideoOutput>& InVideoOutput)
{
	// Linux already populates Buffer via SetupOutputTextureSample, nothing to do
}

#endif // PLATFORM_LINUX
