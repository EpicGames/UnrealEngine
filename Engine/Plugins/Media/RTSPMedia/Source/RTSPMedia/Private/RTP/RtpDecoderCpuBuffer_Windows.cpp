// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/Platform.h"

#if PLATFORM_WINDOWS

#include "RtpDecoderCpuBuffer.h"

#include "RtspMediaConstants.h"

#include "ElectraTextureSample.h"
#include "IElectraDecoderOutputVideo.h"
#include "IElectraDecoderFeaturesAndOptions.h"

#include "Misc/ScopeExit.h"

#include "Windows/AllowWindowsPlatformTypes.h"
#include <mfapi.h>
#include <mfidl.h>
#include <mfobjects.h>
#include "Windows/HideWindowsPlatformTypes.h"

void RtspMedia::PopulateCpuBuffer(FElectraTextureSample* InTextureSample, const TSharedPtr<IElectraDecoderVideoOutput>& InVideoOutput)
{
	if (!InTextureSample || !InVideoOutput.IsValid())
	{
		return;
	}

	// On Windows with DX12 GPU decode, SetupOutputTextureSample populates the GPU texture.
	// The MFSample is also available on the output when force_cpu_output is set.
	if (InTextureSample->Buffer.IsValid())
	{
		return;
	}

	void* MFSampleHandle = InVideoOutput->GetPlatformOutputHandle(EElectraDecoderPlatformOutputHandleType::MFSample);
	if (!MFSampleHandle)
	{
		UE_LOGF(LogRtspMedia, VeryVerbose, "PopulateCpuBuffer: No MFSample available on video output");
		return;
	}

	TRefCountPtr<IMFSample> Sample = static_cast<IMFSample*>(MFSampleHandle);
	TRefCountPtr<IMFMediaBuffer> MediaBuffer;
	HRESULT Result = Sample->GetBufferByIndex(0, MediaBuffer.GetInitReference());
	if (FAILED(Result))
	{
		UE_LOGF(LogRtspMedia, Warning, "PopulateCpuBuffer: GetBufferByIndex failed (0x%08X)", static_cast<uint32>(Result));
		return;
	}

	TRefCountPtr<IMF2DBuffer> Buffer2D;
	Result = MediaBuffer->QueryInterface(__uuidof(IMF2DBuffer), reinterpret_cast<void**>(Buffer2D.GetInitReference()));
	if (FAILED(Result))
	{
		UE_LOGF(LogRtspMedia, Warning, "PopulateCpuBuffer: IMF2DBuffer query failed (0x%08X)", static_cast<uint32>(Result));
		return;
	}

	uint8* Data = nullptr;
	LONG Pitch = 0;
	Result = Buffer2D->Lock2D(&Data, &Pitch);
	if (FAILED(Result))
	{
		UE_LOGF(LogRtspMedia, Warning, "PopulateCpuBuffer: Lock2D failed (0x%08X)", static_cast<uint32>(Result));
		return;
	}

	ON_SCOPE_EXIT
	{
		Buffer2D->Unlock2D();
	};

	if (!Data || Pitch <= 0)
	{
		UE_LOGF(LogRtspMedia, Warning, "PopulateCpuBuffer: Lock2D returned invalid data (Data=%p, Pitch=%d)", Data, static_cast<int32>(Pitch));
		return;
	}

	// NV12: Height * 3/2 to include both Y and UV planes
	const int32 DecodedHeight = InVideoOutput->GetDecodedHeight();
	const int64 TotalBytes = static_cast<int64>(Pitch) * DecodedHeight * 3 / 2;

	if (TotalBytes <= 0)
	{
		UE_LOGF(LogRtspMedia, Warning, "PopulateCpuBuffer: Invalid buffer dimensions (Pitch=%d, DecodedHeight=%d)", static_cast<int32>(Pitch), DecodedHeight);
		return;
	}

	InTextureSample->Buffer = MakeShared<TArray64<uint8>, ESPMode::ThreadSafe>();
	InTextureSample->Buffer->SetNumUninitialized(TotalBytes);
	FMemory::Memcpy(InTextureSample->Buffer->GetData(), Data, TotalBytes);
	InTextureSample->Stride = static_cast<uint32>(Pitch);
}

#endif // PLATFORM_WINDOWS
