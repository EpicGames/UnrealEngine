// Copyright Epic Games, Inc. All Rights Reserved.

#include "RtpH264Depacketizer.h"

#include "RtspMediaConstants.h"

void FRtpH264Depacketizer::ProcessRtpPayload(TArray<uint8> InRtpPayload, uint16 InSequenceNumber, uint32 InTimestamp, bool bInMarker)
{
	if (InRtpPayload.IsEmpty())
	{
		UE_LOGF(LogRtspMedia, Warning, "H264 depacketizer was provided with an empty payload");
		ResetFragmentBuffer();
		return;
	}

	if (!bIsFirstPacket)
	{
		const uint16 ExpectedSequenceNumber = LastSequenceNumber + 1;
		if (InSequenceNumber != ExpectedSequenceNumber)
		{
			UE_LOGF(LogRtspMedia, Warning, "H264 depacketizer expected sequence number %d but got %d", ExpectedSequenceNumber, InSequenceNumber);
			// Drop any partial fragments we might have at this point.
			ResetFragmentBuffer();
			// Notify downstream so it can drop any partially-assembled access unit and
			// flush decoder state before resuming on the next IDR.
			OnNalUnitSequenceGap.ExecuteIfBound();
		}
	}
	else 
	{
		bIsFirstPacket = false;
	}

	LastSequenceNumber = InSequenceNumber;

	// First byte contains the NAL header:
	// F - Forbidden zero bit (always 0)
	// NRI - 2 bits - Importance indicator (0-3)
	// Type - 5 bits - NAL packetization type
	// Clear away the F and NRI bits with 0x1F (00011111) to get the packetization type.
	const uint8 PacketizationType = InRtpPayload[0] & 0x1F;

	// 1 - 23 are complete NAL units
	if (PacketizationType > 0 && PacketizationType < 24)
	{
		HandleSingleUnit(InRtpPayload, InTimestamp, bInMarker);
	}
	// 24 is a RTP specific type indicating STAP-A (Aggregate unit)
	else if (PacketizationType == 24)
	{
		HandleAggregate(InRtpPayload, InTimestamp, bInMarker);
	}
	// 28 is another RTP specific type indicating FU-A (Fragment unit)
	else if (PacketizationType == 28)
	{
		HandleFragment(InRtpPayload, InTimestamp, bInMarker);
	}
	else
	{
		UE_LOGF(LogRtspMedia, Warning, "H264 depacketizer encountered unknown NAL unit type: %d", PacketizationType);

		// Only reset if we're expecting another fragment.
		// It could be a NalType that we don't support. e.g. STAP-B, MTAP, FU-B.
		if (bFragmentInProgress)
		{
			ResetFragmentBuffer();
		}
	}
}

void FRtpH264Depacketizer::SetMaxFragmentBufferSize(int64 InMaxFragmentBufferSize)
{
	if (ensureMsgf(InMaxFragmentBufferSize > 0, TEXT("Attempted to set a zero or negative max fragment buffer size")))
	{
		MaxFragmentBufferSize = InMaxFragmentBufferSize;
	}
}

void FRtpH264Depacketizer::ResetFragmentBuffer()
{
	FragmentBuffer.Empty();
	bFragmentInProgress = false;
	FragmentTimestamp = 0;
}

// Some senders (E.g. The RootEncoder library on Android) incorrectly include Annex B start codes at the
// start of certain NAL units.
// This could be the four byte start code 0x00 0x00 0x00 0x01
// Or the three byte start code 0x00 0x00 0x01
TArrayView<const uint8> FRtpH264Depacketizer::StripAnnexBStartCodes(TArrayView<const uint8> InNalData)
{
	const uint8* Data = InNalData.GetData();
	const int32 Size = InNalData.Num();

	// 4 byte start code
	if (Size >= 4 && Data[0] == 0x00 && Data[1] == 0x00 && Data[2] == 0x00 && Data[3] == 0x01)
	{
		UE_LOGF(LogRtspMedia, VeryVerbose, "Stripping 4 byte Annex B start code from non-compliant NAL unit");
		return TArrayView<const uint8>(Data + 4, Size - 4);
	}

	// 3 byte start code
	if (Size >= 3 && Data[0] == 0x00 && Data[1] == 0x00 && Data[2] == 0x01)
	{
		UE_LOGF(LogRtspMedia, VeryVerbose, "Stripping 3 byte Annex B start code from non-compliant NAL unit");
		return TArrayView<const uint8>(Data + 3, Size - 3);
	}

	return InNalData;
}

void FRtpH264Depacketizer::HandleSingleUnit(TArrayView<const uint8> InPayload, uint32 InTimestamp, bool bInMarker)
{
	EmitNalUnit(InPayload, InTimestamp, bInMarker);
}

void FRtpH264Depacketizer::HandleAggregate(TArrayView<const uint8> InPayload, uint32 InTimestamp, bool bInMarker)
{
	const uint8* Data = InPayload.GetData();
	const int32 DataSize = InPayload.Num();

	// Skip over the NAL type byte we already used to determine that this is an aggregate packet.
	int32 Offset = 1;

	while (Offset + 2 <= DataSize)
	{
		// Multi-byte values are in network byte order.
		const uint16 NalSize = (static_cast<uint16>(Data[Offset]) << 8) | Data[Offset + 1];
		Offset += 2;

		if (NalSize == 0)
		{
			UE_LOGF(LogRtspMedia, Warning, "Detected NAL size of zero within aggregate packet");
			continue;
		}

		if (Offset + NalSize > DataSize)
		{
			UE_LOGF(LogRtspMedia, Warning, "H264 depacketizer encountered malformed aggregate payload");
			break;
		}

		const TArrayView<const uint8> NalData(Data + Offset, NalSize);

		// Only the last NAL in the aggregate can potentially be marked as the last unit of the frame.
		bool bIsLastNal = false;
		if (Offset + NalSize >= DataSize)
		{
			bIsLastNal = true;
		}
		EmitNalUnit(NalData, InTimestamp, bIsLastNal && bInMarker);
		Offset += NalSize;
	}
}

void FRtpH264Depacketizer::HandleFragment(TArrayView<const uint8> InPayload, uint32 InTimestamp, bool bInMarker)
{
	const uint8* Data = InPayload.GetData();
	const int32 DataSize = InPayload.Num();

	if (DataSize < 2)
	{
		UE_LOGF(LogRtspMedia, Warning, "H264 depacketizer encountered fragment payload containing less than 2 bytes");
		return;
	}

	const uint8 FragmentIndicator = Data[0];
	const uint8 FragmentHeader = Data[1];

	// Fragment header contains
	// Start bit (answers the question: is this the first fragment of a NAL unit?)
	// End bit
	// Reserved bit
	// Original NAL unit type - 5 bits (1 - 23)
	const bool bIsStart = (FragmentHeader >> 7) & 0x01;
	const bool bIsEnd = (FragmentHeader >> 6) & 0x01;

	constexpr uint32 FragmentOffset = 2;
	const TArrayView<const uint8> FragmentData(Data + FragmentOffset, DataSize - FragmentOffset);

	if (bIsStart)
	{
		ResetFragmentBuffer();
		// When packetized as FU-A the original NAL header byte is not sent.
		// Instead, the information contained within is split between the first fragment
		// indicator byte and the fragment header in this fragment payload.
		// We take the first 3 bits from the indicator and the last 5 bits from the header.
		// This gives us the original NAL unit header required at the start of a NAL unit for decoding.
		const uint8 NalType = FragmentHeader & 0x1F;
		const uint8 ReconstructedNalHeader = (FragmentIndicator & 0xE0) | NalType;
		FragmentBuffer.Add(ReconstructedNalHeader);
		bFragmentInProgress = true;

		// We use the timestamp of the first fragment
		FragmentTimestamp = InTimestamp;
	}

	if (bFragmentInProgress)
	{
		// Check if we're about to exceed our max fragment buffer size
		if (static_cast<int64>(FragmentBuffer.Num()) + static_cast<int64>(FragmentData.Num()) > MaxFragmentBufferSize)
		{
			UE_LOGF(LogRtspMedia, Warning, "Max fragment buffer size (%lld) exceeded. Dropping fragmented NAL unit.", MaxFragmentBufferSize);
			ResetFragmentBuffer();
			return;
		}
		FragmentBuffer.Append(FragmentData.GetData(), FragmentData.Num());
	}

	if (bIsEnd && bFragmentInProgress)
	{
		EmitNalUnit(FragmentBuffer, FragmentTimestamp, bInMarker);
		ResetFragmentBuffer();
	}
}

void FRtpH264Depacketizer::EmitNalUnit(TArrayView<const uint8> InNalUnit, uint32 InTimestamp, bool bInMarker)
{
	if (!ensureMsgf(!InNalUnit.IsEmpty(), TEXT("NAL unit is empty at point of emission from depacketizer")))
	{
		return;
	}

	// Remove any Annex B start codes that a non-compliant sender may have included in the NAL unit.
	// These shouldn't be present in RTP packetized NAL units. 
	// https://www.rfc-editor.org/rfc/rfc6184.txt#:~:text=Scope%0A%0A%20%20%20This%20payload-,specification,-can%20only%20be
	InNalUnit = StripAnnexBStartCodes(InNalUnit);
	
	if (InNalUnit.IsEmpty())
	{
		UE_LOGF(LogRtspMedia, VeryVerbose, "Non-compliant NAL unit is empty after stripping annex B start codes");
		return;
	}

	// Determine the NAL type from the first byte (last 5 bits).
	// 0x1F = 00011111
	const uint8 NalType = InNalUnit[0] & 0x1F;

	// The final NAL unit output needs to be in the AVCC format with the NAL unit data itself
	// prefixed with a 4-byte big endian value describing the length of the contained NAL unit.
	const int32 NalUnitSize = InNalUnit.Num();
	const uint8 HeaderByte0 = (NalUnitSize >> 24) & 0xFF;
	const uint8 HeaderByte1 = (NalUnitSize >> 16) & 0xFF;
	const uint8 HeaderByte2 = (NalUnitSize >> 8) & 0xFF;
	const uint8 HeaderByte3 = NalUnitSize & 0xFF;

	TArray<uint8> AvccNalUnit;
	AvccNalUnit.Add(HeaderByte0);
	AvccNalUnit.Add(HeaderByte1);
	AvccNalUnit.Add(HeaderByte2);
	AvccNalUnit.Add(HeaderByte3);
	AvccNalUnit.Append(InNalUnit.GetData(), NalUnitSize);

	FRtpH264NalUnit NalUnit;
	NalUnit.Data = MoveTemp(AvccNalUnit);
	NalUnit.Timestamp = InTimestamp;
	NalUnit.NalType = NalType;
	NalUnit.bIsIDRFrame = NalType == 5;
	NalUnit.bIsMarkerBitSet = bInMarker;
	NalUnit.bIsSlice = NalType >= 1 && NalType <= 5;

	if (NalType == 7)
	{
		NalUnit.SequenceParameterSet = TArray<uint8>(InNalUnit.GetData(), InNalUnit.Num());
	}
	else if (NalType == 8)
	{
		NalUnit.PictureParameterSet = TArray<uint8>(InNalUnit.GetData(), InNalUnit.Num());
	}

	OnH264NalUnit.ExecuteIfBound(MoveTemp(NalUnit));
}
