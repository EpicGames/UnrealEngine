// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RtpH264NalUnit.h"

#include "RtspMediaDefaults.h"

DECLARE_DELEGATE_OneParam(FOnH264NalUnit, FRtpH264NalUnit);
DECLARE_DELEGATE(FOnNalUnitSequenceGap);

class FRtpH264Depacketizer
{
public:
	void ProcessRtpPayload(TArray<uint8> InRtpPayload, uint16 InSequenceNumber, uint32 InTimestamp, bool bInMarker);
	void SetMaxFragmentBufferSize(int64 InMaxFragmentBufferSize);

	FOnH264NalUnit OnH264NalUnit;
	FOnNalUnitSequenceGap OnNalUnitSequenceGap;
private:
	void HandleSingleUnit(TArrayView<const uint8> InPayload, uint32 InTimestamp, bool bInMarker); // Complete NAL
	void HandleAggregate(TArrayView<const uint8> InPayload, uint32 InTimestamp, bool bInMarker); // STAP-A containing multiple NALs
	void HandleFragment(TArrayView<const uint8> InPayload, uint32 InTimestamp, bool bInMarker); // FU-A NAL fragment

	void EmitNalUnit(TArrayView<const uint8> InNalUnit, uint32 InTimestamp, bool bInMarker);

	void ResetFragmentBuffer();

	TArrayView<const uint8> StripAnnexBStartCodes(TArrayView<const uint8> InNalData);

	// State for fragment reassembly
	TArray<uint8> FragmentBuffer;
	int64 MaxFragmentBufferSize = RtspMedia::Default::MaxFragmentBufferSizeBytes;
	bool bFragmentInProgress = false;
	uint32 FragmentTimestamp = 0;

	// Sequence tracking
	uint16 LastSequenceNumber = 0;
	bool bIsFirstPacket = true;
};