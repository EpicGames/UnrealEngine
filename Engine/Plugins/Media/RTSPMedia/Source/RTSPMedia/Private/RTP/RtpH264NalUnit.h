// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

struct FRtpH264NalUnit
{
	TArray<uint8> Data;                    // AVCC-formatted NAL unit (for frame buffer)
	TArray<uint8> SequenceParameterSet;    // Raw SPS bytes (non-empty only for NAL type 7)
	TArray<uint8> PictureParameterSet;     // Raw PPS bytes (non-empty only for NAL type 8)
	uint32 Timestamp = 0;
	uint8 NalType = 0;
	bool bIsIDRFrame = false;
	bool bIsMarkerBitSet = false;
	bool bIsSlice = false;
};
