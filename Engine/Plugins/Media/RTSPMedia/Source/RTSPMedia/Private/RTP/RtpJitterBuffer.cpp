// Copyright Epic Games, Inc. All Rights Reserved.

#include "RtpJitterBuffer.h"

#include "RtspMediaConstants.h"

#include "HAL/PlatformTime.h"

bool FRtpJitterBuffer::Initialize(const uint32 InBufferDepthMs, const uint32 InClockRate)
{
	if (InClockRate == 0)
	{
		UE_LOGF(LogRtspMedia, Error, "Attempted to initialize RTP jitter buffer with a zero clock rate");
		return false;
	}

	ClockRate = InClockRate;
	PacketBufferDepthSeconds = InBufferDepthMs / 1000.0f;

	ResetBuffer();
	ResetPacing();

	bInitialized = true;

	UE_LOGF(LogRtspMedia, Verbose, "Initialized RTP jitter buffer with buffer depth: %.1fms Clock rate: %u", PacketBufferDepthSeconds * 1000.0f, ClockRate);
	return true;
}

void FRtpJitterBuffer::EnqueuePacket(FRtpPacket InPacket)
{
	if (!ensureMsgf(bInitialized, TEXT("RTP jitter must be initialized before enqueuing packets")))
	{
		return;
	}

	// If we're in the filling phase and we haven't enqueued a packet yet
	if (bFilling && !bFirstPacketEnqueued)
	{
		FillStartTimeSeconds = FPlatformTime::Seconds();
	}

	// First packet or a packet with a distinct timestamp
	const uint32 Timestamp = InPacket.Header.Timestamp;
	if (!bFirstPacketEnqueued || Timestamp != LastEnqueuedTimestamp)
	{
		++QueuedFrameCount;
	}

	UE_LOGF(
		LogRtspMedia,
		VeryVerbose,
		"RTP jitter buffer enqueuing packet with sequence number: %u RTP timestamp: %u",
		InPacket.Header.SequenceNumber,
		InPacket.Header.Timestamp);

	PacketBuffer.Emplace(MoveTemp(InPacket));

	LastEnqueuedTimestamp = Timestamp;
	bFirstPacketEnqueued = true;
}

bool FRtpJitterBuffer::DequeueIfReady(FRtpPacket& OutPacket)
{
	if (!ensureMsgf(bInitialized, TEXT("RTP jitter must be initialized before dequeuing packets")))
	{
		return false;
	}

	if (PacketBuffer.IsEmpty())
	{
		return false;
	}

	if (!ensureMsgf(QueueIndex < PacketBuffer.Num(), TEXT("QueueIndex: %d exceeds the number of packets in the jitter buffer: %d Discarding pending packets"), QueueIndex, PacketBuffer.Num()))
	{
		ResetBuffer();
		ResetPacing();
		return false;
	}

	const double Now = FPlatformTime::Seconds();

	FRtpPacket& FirstPacket = PacketBuffer[QueueIndex];

	// Skip buffer filling and pacing if the configured packet buffer depth is 0
	if (PacketBufferDepthSeconds > 0.0f)
	{
		if (bFilling)
		{
			if (!bFirstPacketEnqueued)
			{
				return false;
			}

			if (Now - FillStartTimeSeconds < PacketBufferDepthSeconds)
			{
				return false;
			}

			// The buffer is full, allow draining to take place.
			NextReleaseTimeSeconds = Now;
			bFilling = false;
		}

		// If the packet contains the same timestamp as the last dequeued packet, release it immediately.
		const bool bSameTimestamp = bFirstPacketDequeued && FirstPacket.Header.Timestamp == LastDequeuedTimestamp;
		if (!bSameTimestamp)
		{
			// If it's not release time yet, abort.
			if (EstimatedFrameIntervalSeconds > 0.0f && Now < NextReleaseTimeSeconds)
			{
				return false;
			}

			// Calculate the initial estimated frame interval and target frame count if not set
			if (EstimatedFrameIntervalSeconds == 0.0f && QueuedFrameCount >= 2)
			{
				// The cast to int32 is intentional for wraparound safety.
				const int32 Span = static_cast<int32>(PacketBuffer.Last().Header.Timestamp - FirstPacket.Header.Timestamp);
				// A non-positive span can occur if all buffered packets share the same RTP timestamp.
				// Skip estimation and defer to the next dequeue attempt when more data is available.
				if (Span > 0)
				{
					EstimatedFrameIntervalSeconds = (static_cast<float>(Span) / ClockRate) / (QueuedFrameCount - 1);
				}
			}

			// Adjust the estimated frame interval for any deviation from the previously calculated frame interval
			if (EstimatedFrameIntervalSeconds > 0.0f)
			{
				const float TargetFrameCount = PacketBufferDepthSeconds / EstimatedFrameIntervalSeconds;

				// Normalized deviation of actual queued frame count vs the target frame count
				const float Error = (static_cast<float>(QueuedFrameCount) - TargetFrameCount) / TargetFrameCount;

				// Prevent the adjustment from becoming negative when an extreme burst of packets is present in the buffer.
				const float Adjustment = FMath::Max(1.0f - BufferLevelGain * Error, BufferLevelGain);

				// Adjust the estimated frame interval by the error to bring it closer to the actual frame interval
				EstimatedFrameIntervalSeconds *= Adjustment;
			}

			// We will release now so compute the next release time
			NextReleaseTimeSeconds += EstimatedFrameIntervalSeconds;
		}
	}

	if (!bFirstPacketDequeued || FirstPacket.Header.Timestamp != LastDequeuedTimestamp)
	{
		--QueuedFrameCount;
	}

	UE_LOGF(
		LogRtspMedia,
		VeryVerbose,
		"Dequeuing RTP packet from jitter buffer. Sequence Number: %u Timestamp: %u",
		FirstPacket.Header.SequenceNumber,
		FirstPacket.Header.Timestamp);

	bFirstPacketDequeued = true;
	LastDequeuedTimestamp = FirstPacket.Header.Timestamp;
	OutPacket = MoveTemp(FirstPacket);
	++QueueIndex;
	CompactPacketBuffer();

	return true;
}

void FRtpJitterBuffer::SetTargetBufferDepth(float InDepthSeconds)
{
	const float PreviousDepthSeconds = PacketBufferDepthSeconds;
	PacketBufferDepthSeconds = FMath::Max(InDepthSeconds, 0.0f);

	if (!FMath::IsNearlyEqual(PacketBufferDepthSeconds, PreviousDepthSeconds, 0.001f))
	{
		UE_LOGF(LogRtspMedia, Verbose, "Jitter buffer depth updated: %.1fms -> %.1fms",
			PreviousDepthSeconds * 1000.0f, PacketBufferDepthSeconds * 1000.0f);
	}
}

int32 FRtpJitterBuffer::NumQueuedPackets() const
{
	return PacketBuffer.Num() - QueueIndex;
}

void FRtpJitterBuffer::CompactPacketBuffer()
{
	if (QueueIndex > NumQueuedPackets())
	{
		PacketBuffer.RemoveAt(0, QueueIndex);
		QueueIndex = 0;
	}

	if (NumQueuedPackets() == 0)
	{
		ResetBuffer();
		ResetPacing();
	}
}

void FRtpJitterBuffer::ResetBuffer()
{
	PacketBuffer.Empty();
	QueueIndex = 0;
}

void FRtpJitterBuffer::ResetPacing()
{
	EstimatedFrameIntervalSeconds = 0.0f;
	bFilling = true;
	FillStartTimeSeconds = 0.0;
	bFirstPacketDequeued = false;
	bFirstPacketEnqueued = false;
	LastDequeuedTimestamp = 0;
	LastEnqueuedTimestamp = 0;
	NextReleaseTimeSeconds = 0.0;
	QueuedFrameCount = 0;
}
