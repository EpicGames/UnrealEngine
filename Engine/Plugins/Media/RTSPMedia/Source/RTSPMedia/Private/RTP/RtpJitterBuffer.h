// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RtpPacket.h"

/**
 * The jitter buffer assumes that packets are enqueued in order.
 * This holds true naturally when the TCP interleaved connection mode is used.
 * For UDP transport packets must be ordered before they are added to the buffer.
 */
class FRtpJitterBuffer
{
public:
	bool Initialize(const uint32 InBufferDepthMs, const uint32 InClockRate);
	void EnqueuePacket(FRtpPacket InPacket);
	bool DequeueIfReady(FRtpPacket& OutPacket);
	void SetTargetBufferDepth(float InDepthSeconds);

private:
	void CompactPacketBuffer();
	int32 NumQueuedPackets() const;
	void ResetBuffer();
	void ResetPacing();

	static constexpr float BufferLevelGain = 0.02f;

	// Configuration
	uint32 ClockRate = 0;
	bool bInitialized = false;
	float PacketBufferDepthSeconds = 0.0f;

	// Buffer
	TArray<FRtpPacket> PacketBuffer;
	int32 QueueIndex = 0;

	// Fill Phase
	bool bFilling = true;
	double FillStartTimeSeconds = 0.0;

	// Frame Pacing
	float EstimatedFrameIntervalSeconds = 0.0f;
	double NextReleaseTimeSeconds = 0.0;
	int32 QueuedFrameCount = 0;

	// Enqueue/Dequeue Tracking
	uint32 LastEnqueuedTimestamp = 0;
	bool bFirstPacketEnqueued = false;
	uint32 LastDequeuedTimestamp = 0;
	bool bFirstPacketDequeued = false;
};
