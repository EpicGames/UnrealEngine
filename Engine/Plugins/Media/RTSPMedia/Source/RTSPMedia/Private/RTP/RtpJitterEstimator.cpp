// Copyright Epic Games, Inc. All Rights Reserved.

#include "RtpJitterEstimator.h"

#include "RtspMediaConstants.h"

#include "HAL/PlatformTime.h"

bool FRtpJitterEstimator::Initialize(uint32 InClockRate, float InObservationWindowSeconds)
{
	if (InClockRate == 0)
	{
		UE_LOGF(LogRtspMedia, Error, "Attempted to initialize jitter estimator with a zero clock rate");
		return false;
	}

	if (InObservationWindowSeconds <= 0.0f)
	{
		UE_LOGF(LogRtspMedia, Error, "Attempted to initialize jitter estimator with a non-positive observation window");
		return false;
	}

	ClockRate = InClockRate;
	ObservationWindowSeconds = InObservationWindowSeconds;
	BaselineSeconds = 0.0;
	RecentMaxLatenessSeconds = 0.0;
	WindowStartTimeSeconds = 0.0;
	LastTimestamp = 0;
	LastArrivalTimeSeconds = 0.0;
	bFirstFrame = true;
	bInitialized = true;

	UE_LOGF(LogRtspMedia, Verbose, "Initialized jitter estimator with clock rate: %u observation window: %.1fs", ClockRate, ObservationWindowSeconds);
	return true;
}

void FRtpJitterEstimator::RecordArrival(uint32 InRtpTimestamp)
{
	if (!bInitialized)
	{
		return;
	}

	// Skip FU-A fragments (same RTP timestamp as previous)
	if (!bFirstFrame && InRtpTimestamp == LastTimestamp)
	{
		return;
	}

	const double Now = FPlatformTime::Seconds();

	if (bFirstFrame)
	{
		LastTimestamp = InRtpTimestamp;
		LastArrivalTimeSeconds = Now;
		WindowStartTimeSeconds = Now;
		bFirstFrame = false;
		return;
	}

	// Expected interval from RTP timestamps
	// int32 cast handles RTP timestamp wraparound
	const double ExpectedInterval = static_cast<double>(static_cast<int32>(InRtpTimestamp - LastTimestamp)) / ClockRate;

	// Negative expected interval indicates a B-frame (presentation timestamp earlier than
	// previous frame, but delivered in decode order). Skip the measurement — timestamp
	// reordering is not network jitter.
	if (ExpectedInterval <= 0.0)
	{
		return;
	}

	const double ActualInterval = Now - LastArrivalTimeSeconds;
	const double Lateness = ActualInterval - ExpectedInterval;

	// Immediate step-up on any spike exceeding baseline
	if (Lateness > BaselineSeconds)
	{
		BaselineSeconds = CeilToGranularity(Lateness);
		WindowStartTimeSeconds = Now;
		RecentMaxLatenessSeconds = 0.0;
	}

	// Track the worst lateness in the current window
	RecentMaxLatenessSeconds = FMath::Max(RecentMaxLatenessSeconds, FMath::Max(Lateness, 0.0));

	// At window boundary: evaluate and possibly step down
	if ((Now - WindowStartTimeSeconds) >= ObservationWindowSeconds)
	{
		if (RecentMaxLatenessSeconds < BaselineSeconds)
		{
			BaselineSeconds = CeilToGranularity(RecentMaxLatenessSeconds);
		}

		// Reset window
		WindowStartTimeSeconds = Now;
		RecentMaxLatenessSeconds = 0.0;
	}

	LastTimestamp = InRtpTimestamp;
	LastArrivalTimeSeconds = Now;
}

float FRtpJitterEstimator::GetTargetBufferDepthSeconds() const
{
	return static_cast<float>(CeilToGranularity(BaselineSeconds * HeadroomMultiplier));
}

double FRtpJitterEstimator::CeilToGranularity(double InSeconds)
{
	if (InSeconds <= 0.0)
	{
		return 0.0;
	}
	return FMath::CeilToDouble(InSeconds / GranularitySeconds) * GranularitySeconds;
}
