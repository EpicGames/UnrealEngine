// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RtspMediaDefaults.h"

/**
 * Observes RTP packet arrival timing to estimate the ideal jitter buffer depth.
 *
 * The estimator compares expected inter-frame intervals (derived from RTP timestamps)
 * against actual wall-clock delivery intervals. The difference ("lateness") reflects
 * network jitter. A baseline buffer depth is maintained that steps up immediately
 * on spikes and steps down conservatively after sustained improvement over observation
 * windows.
 *
 * Start at zero: baseline begins at 0 (passthrough). Clean networks stay at zero latency.
 * Spikes drive the baseline up immediately.
 * At window boundaries, baseline clamps down to the worst observed lateness in that window.
 * Baseline values are rounded up to a 10ms granularity to avoid minor oscillations.
 *
 * GetTargetBufferDepthSeconds() returns the baseline with a headroom multiplier applied.
 * During a stall, the buffer drains at the playback rate — without headroom the buffer
 * empties right as the burst arrives, causing a visible stutter.
 */
class FRtpJitterEstimator
{
public:
	bool Initialize(uint32 InClockRate, float InObservationWindowSeconds);
	void RecordArrival(uint32 InRtpTimestamp);
	float GetTargetBufferDepthSeconds() const;

private:
	/** Round up to the nearest 10ms. Returns 0 for values at or below 0. */
	static double CeilToGranularity(double InSeconds);

	static constexpr double GranularitySeconds = 0.01;
	static constexpr double HeadroomMultiplier = 2.0;

	uint32 ClockRate = 0;
	bool bInitialized = false;
	double BaselineSeconds = 0.0;
	float ObservationWindowSeconds = RtspMedia::Default::JitterBufferObservationWindowSeconds;
	double RecentMaxLatenessSeconds = 0.0;
	double WindowStartTimeSeconds = 0.0;
	uint32 LastTimestamp = 0;
	double LastArrivalTimeSeconds = 0.0;
	bool bFirstFrame = true;
};
