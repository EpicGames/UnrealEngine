// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

struct FRtpHeader 
{
	// 2 bits - value should always be 2
	uint8 Version = 0;

	// P bit
	bool bPadding = false;

	// X bit
	bool bExtension = false;

	// 4 bits - Contributing source count
	// Used when sources are mixed by a server
	// Typically 0 in video streaming 
	uint8 CsrcCount = 0;

	// M bit - indicates last packet of frame
	bool bMarker = false;

	// 7 bits
	uint8 PayloadType = 0;

	// 2 bytes - for ordering
	uint16 SequenceNumber = 0;

	// 4 bytes - 90kHz clock
	uint32 Timestamp = 0;

	// 4 bytes - Synchronization source ID
	// Random identifier
	uint32 Ssrc = 0;
};

struct FRtpPacket
{
	FRtpHeader Header;
	TArray<uint8> Payload;
};
