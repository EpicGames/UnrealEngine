// Copyright Epic Games, Inc. All Rights Reserved.

#include "RtpParser.h"

#include "RtspMediaConstants.h"

bool FRtpParser::Parse(const TArrayView<const uint8> InData, FRtpHeader& OutHeader, TArrayView<const uint8>& OutPayload)
{
	const int32 DataSize = InData.Num();

	constexpr int32 MinimumHeaderSize = 12;

	// 12 Bytes is the minimum header size
	if (DataSize < MinimumHeaderSize)
	{
		UE_LOGF(LogRtspMedia, Error, "RTP packet size is below the minimum header size: %d", MinimumHeaderSize);
		return false;
	}

	FRtpHeader Header;

	const uint8* Data = InData.GetData();

	// Byte 0 - Version, bPadding, bExtension, CsrcCount
	const uint8 Byte0 = Data[0];
	Header.Version = (Byte0 >> 6) & 0x03;
	Header.bPadding = (Byte0 >> 5) & 0x01;
	Header.bExtension = (Byte0 >> 4) & 0x01;
	Header.CsrcCount = Byte0 & 0x0F;

	if (Header.Version != 2)
	{
		UE_LOGF(LogRtspMedia, Error, "RTP packet version must be '2' instead we received '%d'", Header.Version);
		return false;
	}

	// Byte 1 - bMarker and PayloadType
	const uint8 Byte1 = Data[1];
	Header.bMarker = (Byte1 >> 7) & 0x01;
	Header.PayloadType = Byte1 & 0x7F;

	// All multi-byte values are in network byte order
	// Bytes 2 - 3: Sequence Number
	Header.SequenceNumber = (static_cast<uint16>(Data[2]) << 8) | Data[3];

	// Bytes 4 - 7: Timestamp
	const uint32 TimestampByte0 = static_cast<uint32>(Data[4]) << 24;
	const uint32 TimestampByte1 = static_cast<uint32>(Data[5]) << 16;
	const uint32 TimestampByte2 = static_cast<uint32>(Data[6]) << 8;
	const uint32 TimestampByte3 = static_cast<uint32>(Data[7]);
	Header.Timestamp = TimestampByte0 | TimestampByte1 | TimestampByte2 | TimestampByte3;

	// Bytes 8 - 11: Ssrc
	const uint32 SsrcByte0 = static_cast<uint32>(Data[8]) << 24;
	const uint32 SsrcByte1 = static_cast<uint32>(Data[9]) << 16;
	const uint32 SsrcByte2 = static_cast<uint32>(Data[10]) << 8;
	const uint32 SsrcByte3 = static_cast<uint32>(Data[11]);
	Header.Ssrc = SsrcByte0 | SsrcByte1 | SsrcByte2 | SsrcByte3;

	// We're not interested in the contributing source identifiers so we skip over those.
	// Each identifier is 4 bytes in size.
	constexpr int32 ContributingSourceIdentifierSize = 4;
	int32 HeaderSize = MinimumHeaderSize + (Header.CsrcCount * ContributingSourceIdentifierSize);

	if (Header.bExtension)
	{
		// Extension Header contains:
		// - Profile ID (2 bytes)
		// - Length (2 bytes) - Number of 32-bit words
		// - Extension size (MinimumExtensionHeaderSize + (Length * 4))
		// We don't use the contents of the extension header, but we need to figure
		// out it's size in order to determine where the payload starts.

		constexpr int32 MinimumExtensionHeaderSize = 4;
		// InData must contain at least the header size and the minimum extension header size bytes
		const int32 MinimumSizeWithExtensionHeader = HeaderSize + MinimumExtensionHeaderSize;
		if (DataSize < MinimumSizeWithExtensionHeader)
		{
			// If not this is a malformed packet
			UE_LOGF(LogRtspMedia, Warning, "RTP packet (size: %d) containing extension header below minimum size: %d", DataSize, MinimumSizeWithExtensionHeader);
			return false; 
		}

		// We skip over Profile ID and go straight to Length (HeaderSize + 2)

		// ExtensionLength is the number of 32-bit words so the number of bytes is (ExtensionLength * 4)
		const uint16 ExtensionLengthByte0 = static_cast<uint16>(Data[HeaderSize + 2]) << 8;
		const uint16 ExtensionLengthByte1 = static_cast<uint16>(Data[HeaderSize + 3]);
		const uint16 ExtensionLength = ExtensionLengthByte0 | ExtensionLengthByte1;

		// Increase the HeaderSize by the 4 byte header + (ExtensionLength * 4) bytes
		HeaderSize += MinimumExtensionHeaderSize + (static_cast<int32>(ExtensionLength) * 4);
	}

	// Final check that data contains at least the size of the header
	if (DataSize < HeaderSize)
	{
		// Malformed packet
		UE_LOGF(LogRtspMedia, Warning, "RTP packet (size: %d) is below expected header size: %d", DataSize, HeaderSize);
		return false;
	}

	OutHeader = Header;
	OutPayload = InData.Slice(HeaderSize, DataSize - HeaderSize);

	if (Header.bPadding)
	{
		if (OutPayload.IsEmpty())
		{
			UE_LOGF(LogRtspMedia, Warning, "RTP packet padding bit is set but the payload is empty. Packet is malformed");
			return false;
		}

		const int32 PayloadSize = OutPayload.Num();
		const uint8 PaddingCount = OutPayload.Last();
		if (PaddingCount > 0 && PaddingCount <= PayloadSize)
		{
			OutPayload = OutPayload.Slice(0, PayloadSize - PaddingCount);
		}
		else 
		{
			UE_LOGF(LogRtspMedia, Warning, "Padding count invalid within RTP packet: %d. Payload size is: %d", PaddingCount, PayloadSize);
			return false;
		}
	}

	return true;
}