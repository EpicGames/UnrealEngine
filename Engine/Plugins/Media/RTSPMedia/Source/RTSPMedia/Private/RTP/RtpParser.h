// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "RtpPacket.h"

class FRtpParser
{
public:
	static bool Parse(const TArrayView<const uint8> InData, FRtpHeader& OutHeader, TArrayView<const uint8>& OutPayload);
};
