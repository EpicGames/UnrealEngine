// Copyright Epic Games, Inc. All Rights Reserved.

#include "RtspClient.h"

#include "RtspMediaConstants.h"

#include "RTP/RtpJitterEstimator.h"
#include "RTP/RtpParser.h"
#include "RTP/RtpPacket.h"

#include "RTSP/RtspMessage.h"
#include "RTSP/RtspTransportConfiguration.h"

#include "SDP/SdpParser.h"
#include "SDP/SdpSession.h"

#include "Async/Async.h"
#include "HAL/PlatformProcess.h"
#include "SocketSubsystem.h"
#include "Sockets.h"
#include "Common/TcpSocketBuilder.h"
#include "RTP/RtpDecoder.h"

FRtspClient::~FRtspClient()
{
	Disconnect();
}

bool FRtspClient::Initialize(const FRtspClientConfiguration& InConfiguration)
{
	if (GetState() != ERtspClientState::Disconnected)
	{
		UE_LOGF(LogRtspMedia, Warning, "Can only initialize when disconnected");
		return false;
	}

	// Validate URL
	const FString Url = InConfiguration.Url;
	if (!ParseUrl(Url, Host, Port, Path))
	{
		UE_LOGF(LogRtspMedia, Error, "Invalid RTSP URL: %ls", *Url);
		return false;
	}

	// Store configuration
	Configuration = InConfiguration;

	FString BaseUrl = FString::Printf(TEXT("rtsp://%s:%d"), *Host, Port);
	FullUrl = Path.IsEmpty() ? BaseUrl : (BaseUrl / Path);

	// Start network thread
	UE_LOGF(LogRtspMedia, Log, "Connecting to %ls", *FullUrl);
	SetState(ERtspClientState::Connecting);

	bStopping.store(false);

	Thread.Reset(FRunnableThread::Create(this, TEXT("RtspClientThread"), 0, TPri_Normal));

	if (!Thread.IsValid())
	{
		UE_LOGF(LogRtspMedia, Error, "Failed to create network thread");
		SetState(ERtspClientState::Error);
		return false;
	}

	return true;
}

void FRtspClient::Disconnect()
{
	if (GetState() == ERtspClientState::Disconnected)
	{
		return;
	}

	UE_LOGF(LogRtspMedia, Log, "Disconnecting from %ls", *FullUrl);

	bStopping.store(true);

	if (Thread.IsValid())
	{
		Thread->Kill(true);
		Thread.Reset();
	}
}

bool FRtspClient::InitializeJitterBuffer(const uint32 InClockRate)
{
	uint32 BufferDepthMs = Configuration.JitterBufferDepthMs;

	if (Configuration.bJitterBufferAutoAdjust)
	{
		JitterEstimator = MakeUnique<FRtpJitterEstimator>();
		if (!JitterEstimator->Initialize(InClockRate, Configuration.JitterBufferObservationWindowSeconds))
		{
			JitterEstimator.Reset();
			return false;
		}
		BufferDepthMs = 0;
	}

	JitterBuffer = MakeUnique<FRtpJitterBuffer>();
	if (!JitterBuffer->Initialize(BufferDepthMs, InClockRate))
	{
		JitterBuffer.Reset();
		JitterEstimator.Reset();
		return false;
	}

	return true;
}

ERtspClientState FRtspClient::GetState() const
{
	// State can be read from both the worker thread and the game thread
	FScopeLock Lock(&StateLock);
	return State;
}

bool FRtspClient::Init()
{
	return true;
}

uint32 FRtspClient::Run()
{
	Socket = TUniquePtr<FSocket>(
		FTcpSocketBuilder(TEXT("RtspSocket"))
			.AsNonBlocking()
			.AsReusable()
			.WithReceiveBufferSize(Configuration.SocketBufferSizeBytes)
			.WithSendBufferSize(Configuration.SocketBufferSizeBytes)
			.Build()
	);

	if (!Socket.IsValid())
	{
		UE_LOGF(LogRtspMedia, Error, "Failed to create socket");
		return HandleRunError();
	}

	// Configure the socket for no delay, which disables Nagle's algorithm.
	// Only really affects outgoing packets, but should speed up the initial handshake.
	Socket->SetNoDelay(true);

	ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get();
	if (SocketSubsystem == nullptr)
	{
		UE_LOGF(LogRtspMedia, Error, "Failed to get socket subsystem");
		return HandleRunError();
	}

	FAddressInfoResult AddressInfoResult = SocketSubsystem->GetAddressInfo(*Host, nullptr, EAddressInfoFlags::Default, NAME_None);
	if (AddressInfoResult.ReturnCode != SE_NO_ERROR || AddressInfoResult.Results.IsEmpty())
	{
		UE_LOGF(LogRtspMedia, Warning, "Failed to resolve host: %ls", *Host);
		return HandleRunError();
	}
	TSharedRef<FInternetAddr> ServerAddress = AddressInfoResult.Results[0].Address;

	ServerAddress->SetPort(Port);

	// Connect to Server
	// We're using a non-blocking socket.
	// FSocket::Connect() returns true for EINPROGRESS/EWOULDBLOCK conditions, which is expected
	// for non-blocking sockets. A false return indicates a genuine failure.
	// (e.g. bad address, network unreachable). We then poll with Wait(WaitForWrite) and confirm via
	// GetConnectionState() to determine the actual connection outcome.
	const bool bConnected = Socket->Connect(*ServerAddress);
	if (!bConnected)
	{
		UE_LOGF(LogRtspMedia, Warning, "Failed to connect socket for host: %ls", *Host);
		return HandleRunError();
	}

	// Wait for write
	if (!Socket->Wait(ESocketWaitConditions::WaitForWrite, FTimespan::FromSeconds(5.0)))
	{
		UE_LOGF(LogRtspMedia, Warning, "Connection timed out for host: %ls", *Host);
		return HandleRunError();
	}

	if (Socket->GetConnectionState() != SCS_Connected)
	{
		UE_LOGF(LogRtspMedia, Warning, "Connection failed for host: %ls", *Host);
		return HandleRunError();
	}

	UE_LOGF(LogRtspMedia, Log, "Connected to host: %ls", *Host);
	SetState(ERtspClientState::Connected);

	ReceiveBuffer.SetNumUninitialized(4096);

	// Main loop

	while (!bStopping.load())
	{
		ProcessState();

		if (GetState() == ERtspClientState::Error)
		{
			break;
		}

		if (Socket->Wait(ESocketWaitConditions::WaitForRead, FTimespan::FromMilliseconds(5)))
		{
			if (!ProcessIncomingData())
			{
				SetState(ERtspClientState::Error);
				continue;
			}
		}

		// Check for waiting packets in the jitter buffer
		if (JitterBuffer.IsValid())
		{
			if (JitterEstimator.IsValid())
			{
				JitterBuffer->SetTargetBufferDepth(JitterEstimator->GetTargetBufferDepthSeconds());
			}

			FRtpPacket BufferedPacket;
			while (JitterBuffer->DequeueIfReady(BufferedPacket))
			{
				// Note: Executed on the client worker thread to avoid a hop to the game thread
				OnRtpPacketReceived.ExecuteIfBound(MoveTemp(BufferedPacket));
			}
		}

		// Check for expired requests
		TArray<FRtspPendingRequest> ExpiredRequests = RequestTracker.GetExpiredRequests(Configuration.RequestTimeoutSeconds);
		if (ExpiredRequests.Num() > 0)
		{
			const FRtspPendingRequest& ExpiredRequest = ExpiredRequests[0];
			const ERtspRequestMethod Method = ExpiredRequest.Request.GetRequestMethod().Get(ERtspRequestMethod::Unknown);
			FString MethodString = FRtspMessage::RequestMethodToString(Method);
			UE_LOGF(LogRtspMedia, Error, "%ls request timed out", *MethodString);
			SetState(ERtspClientState::Error);
		}

		if (GetState() == ERtspClientState::Playing)
		{
			// Send keep alive
			const double Now = FPlatformTime::Seconds();
			if (Now - LastKeepAlive > KeepAliveInterval)
			{
				if (!SendKeepAlive())
				{
					SetState(ERtspClientState::Error);
				}
			}
		}
		else 
		{
			FPlatformProcess::Sleep(0.001f);
		}
	}
	
	TeardownConnection();

	SetState(ERtspClientState::Disconnected);

	UE_LOGF(LogRtspMedia, Verbose, "Network thread exiting");
	
	return 0;
}

void FRtspClient::Stop()
{
	bStopping.store(true);
}

const TCHAR* FRtspClient::StateToString(ERtspClientState InState)
{
	switch (InState)
	{
		case ERtspClientState::Disconnected: return TEXT("Disconnected");
		case ERtspClientState::Connecting: return TEXT("Connecting");
		case ERtspClientState::Connected: return TEXT("Connected");
		case ERtspClientState::Ready: return TEXT("Ready");
		case ERtspClientState::Described: return TEXT("Described");
		case ERtspClientState::Setup: return TEXT("Setup");
		case ERtspClientState::Playing: return TEXT("Playing");
		case ERtspClientState::Error: return TEXT("Error");
		default: return TEXT("Unknown");
	}
}

void FRtspClient::SetState(ERtspClientState InNewState)
{
	ERtspClientState OldState;
	{
		FScopeLock Lock(&StateLock);

		if (State == InNewState)
		{
			return;
		}

		OldState = State;
		State = InNewState;
	}

	UE_LOGF(LogRtspMedia, Verbose, "State changed: %ls -> %ls", StateToString(OldState), StateToString(InNewState));

	AsyncTask(ENamedThreads::GameThread, [Delegate = OnStateChanged, InNewState, OldState]()
	{
		Delegate.ExecuteIfBound(InNewState, OldState);
	});
}

void FRtspClient::ProcessState()
{
	// Check if we have any requests in flight, if so wait until the response has been
	// handled or times out before performing the next state transition.
	if (RequestTracker.HasPendingRequests())
	{
		return;
	}
	
	ERtspClientState CurrentState = GetState();
	switch (CurrentState) 
	{
		// Just connected. Send OPTIONS to query server capabilities.
		case ERtspClientState::Connected:
			if (!SendOptions())
			{
				SetState(ERtspClientState::Error);
			}
			break;
		// OPTIONS succeeded. Send DESCRIBE to get SDP (Session Description Protocol) description data
		case ERtspClientState::Ready:
			if (!SendDescribe())
			{
				SetState(ERtspClientState::Error);
			}
			break;
		// DESCRIBE succeeded. Send SETUP for each track
		// Basic initial implementation, just request the first video track
		case ERtspClientState::Described:
			if (SdpSession.IsValid() && SdpSession->VideoTracks.Num() > 0)
			{
				const FSdpMediaTrack& Track = SdpSession->VideoTracks[0];
				if (!SendSetup(Track.ControlUrl, 0))
				{
					SetState(ERtspClientState::Error);
				}
			}
			else 
			{
				UE_LOGF(LogRtspMedia, Error, "No video tracks in session description");
				SetState(ERtspClientState::Error);
			}
			break;
		// SETUP succeeded. Send PLAY to start streaming.
		case ERtspClientState::Setup:
			if (!SendPlay())
			{
				SetState(ERtspClientState::Error);
			}
			break;
		case ERtspClientState::Playing:
		case ERtspClientState::Error:
		case ERtspClientState::Disconnected:
		case ERtspClientState::Connecting:
			// No state transition to make
			break;
	}
}

bool FRtspClient::ProcessIncomingData()
{
	int32 BytesRead = 0;
	const bool bSuccess = Socket->Recv(ReceiveBuffer.GetData(), ReceiveBuffer.Num(), BytesRead);

	if (!bSuccess)
	{
		UE_LOGF(LogRtspMedia, Warning, "Failed to receive data from socket");
		return false;
	}

	if (BytesRead == 0)
	{
		UE_LOGF(LogRtspMedia, Warning, "Connection closed by server");
		return false;
	}

	// 64 KB Max Process Buffer Size
	constexpr int32 MaxProcessBufferSize = 64 * 1024;
	if (ProcessBuffer.Num() + BytesRead > MaxProcessBufferSize)
	{
		UE_LOGF(LogRtspMedia, Error, "Max RTSP process buffer size (%d) exceeded.", MaxProcessBufferSize);
		ProcessBuffer.Reset();
		return false;
	}

	ProcessBuffer.Append(ReceiveBuffer.GetData(), BytesRead);
	
	while (!ProcessBuffer.IsEmpty())
	{
		const int32 ProcessBufferSize = ProcessBuffer.Num();
		
		if (ProcessBuffer[0]  == '$')
		{
			ProcessInterleavedPacket();
		}
		else
		{
			if (!ProcessRtspMessage())
			{
				return false;
			}
		}

		// If the buffer size has not reduced then we have incomplete RTP or RTSP message data
		// and need to read more in the next iteration.
		if (ProcessBufferSize == ProcessBuffer.Num())
		{
			break;
		}
	}

	return true;
}

void FRtspClient::ProcessInterleavedPacket()
{
	constexpr int32 HeaderLength = 4;
	if (ProcessBuffer.Num() < HeaderLength)
	{
		// Incomplete data
		// We want the run loop to continue and read more data.
		return;
	}

	// Index 0 is the interleaved packet indicator '$'

	// Channel lies at index 1
	const uint8 Channel = ProcessBuffer[1];

	// Packet length is at index 2
	// Data is big endian (Network byte order)
	const uint16 InterleavedPayloadLengthByte0 = static_cast<uint16>(ProcessBuffer[2]) << 8;
	const uint16 InterleavedPayloadLengthByte1 = static_cast<uint16>(ProcessBuffer[3]);
	const uint16 InterleavedPayloadLength = InterleavedPayloadLengthByte0 | InterleavedPayloadLengthByte1;

	if (ProcessBuffer.Num() < HeaderLength + InterleavedPayloadLength)
	{
		// Incomplete data
		// We want the run loop to continue and read more data.
		return;
	}
	
	// TODO: Map channel and track information when audio support is added.
	// Different channels represent different streams. Typically:
	// 0 - Video RTP
	// 1 - Video RTCP
	// 2 - Audio RTP
	// 3 - Audio RTCP
	// We only support the video RTP channel
	if (Channel == 0)
	{
		const TArrayView<const uint8> InterleavedPayload(ProcessBuffer.GetData() + HeaderLength, InterleavedPayloadLength);
		
		FRtpHeader Header;
		TArrayView<const uint8> RtpPayload;
		if (FRtpParser::Parse(InterleavedPayload, Header, RtpPayload))
		{
			if (JitterEstimator.IsValid())
			{
				JitterEstimator->RecordArrival(Header.Timestamp);
			}

			if (JitterBuffer.IsValid())
			{
				FRtpPacket Packet{Header, TArray<uint8>(RtpPayload)};
				JitterBuffer->EnqueuePacket(MoveTemp(Packet));
			}
			else
			{
				UE_LOGF(LogRtspMedia, Verbose, "Jitter buffer not initialized when processing interleaved RTP packet");
			}
		}
		else
		{
			UE_LOGF(LogRtspMedia, Warning, "Failed to parse RTP packet");
		}
	}
	else
	{
		UE_LOGF(LogRtspMedia, VeryVerbose, "Dropping unsupported packet on channel %d", Channel);
	}

	// Once enqueued and copied to the jitter buffer remove those bytes from our buffer
	// Or, if we're dropping the packet, remove it from the buffer.
	ProcessBuffer.RemoveAt(0, HeaderLength + InterleavedPayloadLength);
}

bool FRtspClient::ProcessRtspMessage()
{
	FRtspMessage Message;
	int32 BytesConsumed = 0;

	switch (FRtspMessage::Parse(ProcessBuffer, Message, BytesConsumed))
	{
		// Cannot parse the message yet, not enough data. Try again on the next iteration.
		case ERtspMessageParseResult::Incomplete:
			return true;
		// An error occurred during parsing, abort.
		case ERtspMessageParseResult::Error:
			return false;
		// Message was successfully parsed, continue.
		case ERtspMessageParseResult::Complete:
			break;
	}
	
	ProcessBuffer.RemoveAt(0, BytesConsumed);
	
	switch (Message.GetMessageType())
	{
		case ERtspMessageType::Unknown:
			UE_LOGF(LogRtspMedia, Error, "Received RTSP message of an unknown type");
			return false;
		case ERtspMessageType::Request:
			return HandleRequest(Message);
		case ERtspMessageType::Response:
			return HandleResponse(Message);
	}

	return true;
}

bool FRtspClient::HandleResponse(const FRtspMessage& InResponse)
{
	if (InResponse.GetMessageType() != ERtspMessageType::Response)
	{
		UE_LOGF(LogRtspMedia, Error, "Handle response called with non-response RTSP message");
		return false;
	}

	UE_LOGF(LogRtspMedia, Verbose, "Received RTSP response:\n%ls-----\n", *InResponse.ToString());
	
	const TOptional<int32> ResponseCommandId = InResponse.GetCommandId();

	if (!ResponseCommandId.IsSet())
	{
		UE_LOGF(LogRtspMedia, Warning, "Failed to process RTSP response with missing command ID")
		return false;
	}

	FRtspPendingRequest PendingRequest;
	if (!RequestTracker.TakePendingRequest(ResponseCommandId.GetValue(), PendingRequest))
	{
		UE_LOGF(LogRtspMedia, Warning, "Received RTSP response with unknown command ID: %d", ResponseCommandId.GetValue());
		// Ignore this unexpected response, don't abort processing.
		return true;
	}

	TOptional<int32> StatusCode = InResponse.GetResponseStatusCode();
	int32 StatusCodeValue = -1;
	bool bSuccess = false;
	if (StatusCode.IsSet())
	{
		StatusCodeValue = StatusCode.GetValue();
		bSuccess = StatusCodeValue >= 200 && StatusCodeValue < 300;
	}

	TOptional<FString> Reason = InResponse.GetResponseReason();
	FString ReasonString;
	if (Reason.IsSet())
	{
		ReasonString = Reason.GetValue();
	}

	FRtspMessage Request = PendingRequest.Request;
	
	TOptional<ERtspRequestMethod> Method = Request.GetRequestMethod();
	if (!Method.IsSet())
	{
		// If our pending request didn't have a method set then we can't process it.
		UE_LOGF(LogRtspMedia, Warning, "Pending request doesn't contain a METHOD");
		return true;
	}
	
	if (!bSuccess)
	{
		FString MethodString = FRtspMessage::RequestMethodToString(Method.GetValue());
		UE_LOGF(LogRtspMedia, Warning, "%ls request failed: %d %ls", *MethodString, StatusCodeValue, *ReasonString);
	}

	switch (Method.GetValue())
	{
		case ERtspRequestMethod::Options:
			return HandleOptionsResponse(StatusCodeValue, bSuccess, Request);
		case ERtspRequestMethod::Describe:
			return HandleDescribeResponse(bSuccess, InResponse, Request);
		case ERtspRequestMethod::Setup:
			return HandleSetupResponse(bSuccess, InResponse);
		case ERtspRequestMethod::Play:
			if (!bSuccess)
			{
				return false;
			}
			UE_LOGF(LogRtspMedia, Log, "Playback started");
			// Prevent unnecessary keep-alive message when playing is started
			LastKeepAlive = FPlatformTime::Seconds();
			SetState(ERtspClientState::Playing);
			break;
		default:
			break;
	}

	return true;
}

bool FRtspClient::HandleOptionsResponse(const int32 InStatusCodeValue, const bool bInSuccess, const FRtspMessage& InOriginalRequest)
{
	// The only differentiator between an initial setup OPTIONS request and
	// the keep-alive OPTIONS request is the presence of the sessionId.
	// This is not present before we have set up a session.
	if (InOriginalRequest.GetSessionId().IsSet())
	{
		if (bInSuccess)
		{
			return true;
		}

		if (InStatusCodeValue == 454)
		{
			UE_LOGF(LogRtspMedia, Error, "Keep-alive failed. Session not found.");
		}
	}
	// Otherwise we're looking at the initial OPTIONS request response when setting up a session.
	else
	{
		if (bInSuccess)
		{
			SetState(ERtspClientState::Ready);
			return true;
		}
	}
	
	return false;
}

bool FRtspClient::HandleDescribeResponse(const bool bInSuccess, const FRtspMessage& InResponse, const FRtspMessage& InOriginalRequest)
{
	if (!bInSuccess)
	{
		return false;
	}
	
	TOptional<FString> ContentBaseUrl = InResponse.GetContentBaseUrl();
	FString ContentBaseUrlString;
	if (ContentBaseUrl.IsSet())
	{
		ContentBaseUrlString = ContentBaseUrl.GetValue();
	}
	else
	{
		// Fall back to the originally requested URL
		ContentBaseUrlString = InOriginalRequest.GetRequestUrl().Get(FString());
	}
	
	// Parse SDP Session
	SdpSession = FSdpParser::Parse(InResponse.GetBody(), ContentBaseUrlString);
	if (!SdpSession.IsValid())
	{
		UE_LOGF(LogRtspMedia, Error, "Failed to parse SDP session");
		return false;
	}

	const FSdpMediaTrack* VideoTrack = SdpSession->GetFirstVideoTrack();
	if (VideoTrack == nullptr)
	{
		UE_LOGF(LogRtspMedia, Error, "No video track found within SDP session");
		return false;
	}
	
	// Initialise jitter buffer in the RTSP client
	if (!InitializeJitterBuffer(VideoTrack->ClockRate))
	{
		return false;
	}

	AsyncTask(ENamedThreads::GameThread, [Delegate = OnSdpSessionReceived, SdpSession = *SdpSession]()
	{
		Delegate.ExecuteIfBound(SdpSession);
	});
	
	SetState(ERtspClientState::Described);
	return true;
}

bool FRtspClient::HandleSetupResponse(const bool bInSuccess, const FRtspMessage& InResponse)
{
	if (!bInSuccess)
	{
		return false;
	}
	
	if (SessionId.IsEmpty())
    {
    	TOptional<FString> ResponseSessionId = InResponse.GetSessionId();
    	if (!ResponseSessionId.IsSet())
    	{
    		UE_LOGF(LogRtspMedia, Error, "No session ID in SETUP response");
    		return false;
    	}

		SessionId = ResponseSessionId.GetValue();
    	UE_LOGF(LogRtspMedia, Verbose, "Session ID: %ls", *SessionId);

    	TOptional<int32> SessionTimeout = InResponse.GetSessionTimeout();
    	if (SessionTimeout.IsSet())
    	{
    		SessionTimeoutSeconds = SessionTimeout.GetValue();
    		KeepAliveInterval = static_cast<double>(SessionTimeoutSeconds) / 2.0;
    		UE_LOGF(LogRtspMedia, Verbose, "SETUP response provided session timeout of %d seconds", SessionTimeoutSeconds);
    	}
    	else
    	{
    		UE_LOGF(LogRtspMedia, Verbose, "No timeout specified in SETUP response. Assuming %d seconds", SessionTimeoutSeconds);
    	}
    	UE_LOGF(LogRtspMedia, Verbose, "Keep-alive interval set to %.2f seconds", KeepAliveInterval);
    }
    
    SetState(ERtspClientState::Setup);
	return true;
}

bool FRtspClient::HandleRequest(const FRtspMessage& InRequest)
{
	if (InRequest.GetMessageType() != ERtspMessageType::Request)
	{
		return false;
	}

	UE_LOGF(LogRtspMedia, Verbose, "Received RTSP request:\n%ls-----\n", *InRequest.ToString());

	const ERtspRequestMethod Method = InRequest.GetRequestMethod().Get(ERtspRequestMethod::Unknown);
	const TOptional<int32> RequestCommandId = InRequest.GetCommandId();

	if (!RequestCommandId.IsSet())
	{
		UE_LOGF(LogRtspMedia, Warning, "Cannot process RTSP request with missing command ID");
		return false;
	}

	const FString RequestMethodString = FRtspMessage::RequestMethodToString(Method);

	UE_LOGF(LogRtspMedia, Verbose, "Received RTSP %ls request from server with command ID: %d", *RequestMethodString, RequestCommandId.GetValue());

	FRtspMessage Response;

	switch (Method)
	{
	case ERtspRequestMethod::GetParameter:
		if (InRequest.GetBody().IsEmpty())
		{
			// RFC 2326 Section 10.8
			// https://datatracker.ietf.org/doc/html/rfc2326#section-10.8
			// Empty body GET_PARAMETER requests are the per-spec way to perform keep alive messaging.
			// Some servers will use this to check if client is still active and interested.
			Response = FRtspMessage::Response(200, TEXT("OK"), RequestCommandId.GetValue());
		}
		else
		{
			// We don't support supplying any actual parameters
			Response = FRtspMessage::Response(451, TEXT("Parameter Not Understood"), RequestCommandId.GetValue());
		}
		break;
	case ERtspRequestMethod::Options:
		// There's a convention that some servers might follow of using OPTIONS requests as keep-alive messages.
		// The OPTIONS response should contain the requests that we can handle.
		// That handling is in this method, so all we support is GET_PARAMETER and OPTIONS
		Response = FRtspMessage::Response(200, TEXT("OK"), RequestCommandId.GetValue());
		Response.SetHeader(TEXT("Public"), TEXT("GET_PARAMETER, OPTIONS"));
		break;
	default:
		UE_LOGF(LogRtspMedia, Verbose, "Received unsupported RTSP request from server: %ls", *RequestMethodString);
		Response = FRtspMessage::Response(501, TEXT("Not Implemented"), RequestCommandId.GetValue());
	}

	// If the sender provides a session ID then we'll mirror this back in the response
	TOptional<FString> RequestSessionId = InRequest.GetSessionId();
	if (RequestSessionId.IsSet())
	{
		FString RequestSessionIdString = RequestSessionId.GetValue();
		bool bSessionNotFound = false;
		
		// We don't have a session ID yet, why did the request contain one?
		if (SessionId.IsEmpty())
		{
			UE_LOGF(LogRtspMedia, Warning, "RTSP request contains a session ID (%ls), but setup is yet to occur.", *RequestSessionIdString);
			bSessionNotFound = true;
		}
		// Check that this matches our known SessionId
		// If it doesn't match something has gone wrong on the server side
		else if (RequestSessionIdString != SessionId)
		{
			UE_LOGF(LogRtspMedia, Warning, "Received request for unknown session ID: %ls Expected: %ls", *RequestSessionIdString, *SessionId);
			bSessionNotFound = true;
		}

		// If session not found replace the pending response with 'Session Not Found'
		if (bSessionNotFound)
		{
			Response = FRtspMessage::Response(454, TEXT("Session Not Found"), RequestCommandId.GetValue());
		}
		else
		{
			Response.SetSession(RequestSessionIdString);	
		}
	}

	return SendMessage(Response);
}

bool FRtspClient::SendOptions()
{
	UE_LOGF(LogRtspMedia, Verbose, "Sending OPTIONS");
	const FRtspMessage Request = FRtspMessage::Request(ERtspRequestMethod::Options, FullUrl, ++CommandId);
	return SendRequest(Request);
}

bool FRtspClient::SendDescribe()
{
	UE_LOGF(LogRtspMedia, Verbose, "Sending DESCRIBE");
	FRtspMessage Request = FRtspMessage::Request(ERtspRequestMethod::Describe, FullUrl, ++CommandId);
	Request.SetAccept(TEXT("application/sdp"));
	return SendRequest(Request);
}

bool FRtspClient::SendSetup(const FString& InControlUrl, const int32 InTrackIndex)
{
	UE_LOGF(LogRtspMedia, Verbose, "Sending SETUP for track %d: %ls", InTrackIndex, *InControlUrl);

	const FString ResolvedControlUrl = SdpSession->ResolveControlUrl(InControlUrl);
	FRtspMessage Request = FRtspMessage::Request(ERtspRequestMethod::Setup, ResolvedControlUrl, ++CommandId);

	if (!SessionId.IsEmpty())
	{
		Request.SetSession(SessionId);
	}

	FRtspTransportConfiguration::TransportProtocol TransportProtocol = Configuration.TransportProtocol;
	check(TransportProtocol == FRtspTransportConfiguration::TransportProtocol::TCP || TransportProtocol == FRtspTransportConfiguration::TransportProtocol::UDP);

	// Each track needs two channels/ports (one for RTP and another for RTCP, even if we're not using the latter)
	// Convention is even number for RTP, odd number for RTCP.
	if (TransportProtocol == FRtspTransportConfiguration::TransportProtocol::TCP)
	{
		const int32 RtpChannel = InTrackIndex * 2;
		const int32 RtcpChannel = RtpChannel + 1;
		Request.SetTransport(FRtspTransportConfiguration::TcpInterleaved(RtpChannel, RtcpChannel));
	}
	else if (TransportProtocol == FRtspTransportConfiguration::TransportProtocol::UDP)
	{
		// Using a fixed based port offset for now.
		// TODO: Dynamic port allocation
		const int32 RtpPort = 50000 + InTrackIndex * 2;
		const int32 RtcpPort = RtpPort + 1;
		Request.SetTransport(FRtspTransportConfiguration::Udp(RtpPort, RtcpPort));
	}

	return SendRequest(Request);
}

bool FRtspClient::SendPlay()
{
	UE_LOGF(LogRtspMedia, Verbose, "Sending PLAY");
	FRtspMessage Request = FRtspMessage::Request(ERtspRequestMethod::Play, FullUrl, ++CommandId);
	Request.SetSession(SessionId);
	return SendRequest(Request);
}

bool FRtspClient::SendTeardown()
{
	UE_LOGF(LogRtspMedia, Verbose, "Sending TEARDOWN");
	FRtspMessage Request = FRtspMessage::Request(ERtspRequestMethod::Teardown, FullUrl, ++CommandId);
	Request.SetSession(SessionId);
	// Using SendMessage here to avoid adding the request to the tracker.
	// We're usually shutting down at this point so we don't need to wait for a response.
	return SendMessage(Request);
}

// We lean on the common convention to use an OPTIONS request containing the session ID as a keep-alive message for the RTSP connection.
// Officially an empty GET_PARAMETER request should be used for keep-alive, but as it's an optional method it's often absent.
bool FRtspClient::SendKeepAlive()
{
	UE_LOGF(LogRtspMedia, Verbose, "Sending keep-alive (OPTIONS)");
	FRtspMessage Request = FRtspMessage::Request(ERtspRequestMethod::Options, FullUrl, ++CommandId);
	Request.SetSession(SessionId);
	LastKeepAlive = FPlatformTime::Seconds();
	return SendRequest(Request);
}

bool FRtspClient::SendMessage(const FRtspMessage& InMessage)
{
	const FString MessageString = InMessage.ToString();
	UE_LOGF(LogRtspMedia, Verbose, "Sending RTSP message:\n%ls-----\n", *MessageString);

	const FTCHARToUTF8 Utf8MessageString(*MessageString);

	const uint8* Data = reinterpret_cast<const uint8*>(Utf8MessageString.Get());
	const int32 Length = Utf8MessageString.Length();
	int32 TotalBytesSent = 0;

	while (TotalBytesSent < Length)
	{
		const bool bReadyToWrite = Socket->Wait(ESocketWaitConditions::WaitForWrite, FTimespan::FromSeconds(1.0));
		if (bReadyToWrite == false) 
		{
			UE_LOGF(LogRtspMedia, Warning, "Write timeout when sending RTSP message: %ls", *MessageString);
			return false;
		}

		int32 BytesSent = 0;
		const bool bSuccess = Socket->Send(Data + TotalBytesSent, Length - TotalBytesSent, BytesSent);
		if (!bSuccess || BytesSent <= 0)
		{
			UE_LOGF(LogRtspMedia, Warning, "Failed to send RTSP message: %ls", *MessageString);
			return false;
		}

		TotalBytesSent += BytesSent;
	}

	return true;
}

bool FRtspClient::SendRequest(const FRtspMessage& InRequest)
{
	const TOptional <int32> RequestCommandId = InRequest.GetCommandId();
	if (!ensureMsgf(RequestCommandId.IsSet(), TEXT("Cannot send request if command ID is not set")))
	{
		return false;
	}
	
	if (SendMessage(InRequest))
	{
		// Track the request so we can handle the response.
		RequestTracker.AddPendingRequest(RequestCommandId.GetValue(), FRtspPendingRequest{InRequest, FPlatformTime::Seconds()});
		return true;
	}
	
	return false;
}

bool FRtspClient::ParseUrl(const FString& InUrl, FString& OutHost, int32& OutPort, FString& OutPath)
{
	// URL should match format: rtsp://host:port/path or rtsp://host/path

	const FString Scheme = "rtsp://";
	if (!InUrl.StartsWith(Scheme, ESearchCase::IgnoreCase))
	{
		return false;
	}

	FString Remainder = InUrl.Mid(Scheme.Len());
	int32 FirstSlashIndex = Remainder.Find(TEXT("/"));
	FString HostPort;

	if (FirstSlashIndex != INDEX_NONE)
	{
		HostPort = Remainder.Left(FirstSlashIndex);
		OutPath = Remainder.Mid(FirstSlashIndex + 1);
	}
	else
	{
		HostPort = Remainder;
		OutPath = FString();
	}

	int32 ColonIndex = HostPort.Find(TEXT(":"));

	int32 DefaultRtspPort = 554;
	if (ColonIndex != INDEX_NONE)
	{
		OutHost = HostPort.Left(ColonIndex);
		OutPort = FCString::Atoi(*HostPort.Mid(ColonIndex + 1));

		if (OutPort <= 0 || OutPort > 65535)
		{
			OutPort = DefaultRtspPort;
		}
	}
	else 
	{
		OutHost = HostPort;
		OutPort = DefaultRtspPort;
	}

	return !OutHost.IsEmpty();
}

void FRtspClient::TeardownConnection()
{
	if (!SessionId.IsEmpty() && Socket.IsValid() && Socket->GetConnectionState() == SCS_Connected)
	{
		SendTeardown();
	}

	if (Socket.IsValid())
	{
		Socket->Close();
		Socket.Reset();
	}

	SessionId.Empty();
	CommandId = 0;
	SdpSession.Reset();

	JitterBuffer.Reset();
	JitterEstimator.Reset();
	RequestTracker.Clear();

	ReceiveBuffer.Reset();
	ProcessBuffer.Reset();
}

int32 FRtspClient::HandleRunError()
{
	TeardownConnection();
	SetState(ERtspClientState::Error);
	return 1;
}
