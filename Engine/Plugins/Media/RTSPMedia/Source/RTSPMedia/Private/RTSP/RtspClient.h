// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RTP/RtpJitterBuffer.h"
#include "RTP/RtpJitterEstimator.h"
#include "RtspMediaConstants.h"
#include "RtspMediaDefaults.h"
#include "RtspMessage.h"
#include "RtspRequestTracker.h"

#include "CoreMinimal.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "Sockets.h"

class FSocket;
class FRtpParser;

struct RtpPacket;
struct FSdpSession;

struct FRtspClientConfiguration
{
	bool bJitterBufferAutoAdjust = RtspMedia::Default::bJitterBufferAutoAdjust;
	uint32 JitterBufferDepthMs = RtspMedia::Default::JitterBufferDepthMs;
	float JitterBufferObservationWindowSeconds = RtspMedia::Default::JitterBufferObservationWindowSeconds;
	int32 SocketBufferSizeBytes = RtspMedia::Default::SocketBufferSizeBytes;
	float RequestTimeoutSeconds = RtspMedia::Default::RequestTimeoutSeconds;
	FRtspTransportConfiguration::TransportProtocol TransportProtocol = FRtspTransportConfiguration::TransportProtocol::TCP;
	FString Url;
};

enum class ERtspClientState : uint8
{
	Disconnected,
	Connecting,
	Connected,
	Ready,
	Described,
	Setup,
	Playing,
	Error
};

// Executed on the game thread
DECLARE_DELEGATE_TwoParams(FOnRtspClientStateChanged, ERtspClientState, ERtspClientState);
DECLARE_DELEGATE_OneParam(FOnSdpSessionReceived, const FSdpSession&);

// Executed on the client worker thread to minimise latency
DECLARE_DELEGATE_OneParam(FOnRtpPacketReceived, FRtpPacket);

/**
 * Uses a dedicated worker thread for:
 * - Socket input and output
 * - State tracking
 * - RTSP message handling
 * - RTP packet handling
 * 
 * The public methods (Initialize, Disconnect) are expected be called on the game thread,
 * and the state changed and SDP session received delegates are triggered on the game thread.
 *
 * The on RTP packet received delegate is executed on the worker thread to avoid a hop
 * to the game thread before the packet is processed by the decoder (which has its own
 * worker thread).
 * 
 */
class FRtspClient : public FRunnable
{
public:
	virtual ~FRtspClient() override;

	bool Initialize(const FRtspClientConfiguration& InConfiguration);
	void Disconnect();

	FOnRtspClientStateChanged OnStateChanged;
	FOnSdpSessionReceived OnSdpSessionReceived;

	FOnRtpPacketReceived OnRtpPacketReceived;

private:
	virtual bool Init() override;
	virtual uint32 Run() override;
	virtual void Stop() override;

	static const TCHAR* StateToString(const ERtspClientState InState);

	ERtspClientState GetState() const;
	void SetState(ERtspClientState InNewState);

	void ProcessState();
	bool ProcessIncomingData();
	void ProcessInterleavedPacket();
	bool ProcessRtspMessage();

	bool HandleResponse(const FRtspMessage& InMessage);
	bool HandleRequest(const FRtspMessage& InRequest);

	bool HandleOptionsResponse(const int32 InStatusCodeValue, const bool bInSuccess, const FRtspMessage& InOriginalRequest);
	bool HandleDescribeResponse(const bool bInSuccess, const FRtspMessage& InResponse, const FRtspMessage& InOriginalRequest);
	bool HandleSetupResponse(const bool bInSuccess, const FRtspMessage& InResponse);

	bool InitializeJitterBuffer(const uint32 InClockRate);

	bool SendOptions();
	bool SendDescribe();
	bool SendSetup(const FString& InControlUrl, const int32 InTrackIndex);
	bool SendPlay();
	bool SendTeardown();
	bool SendKeepAlive();

	bool SendMessage(const FRtspMessage& InMessage);
	bool SendRequest(const FRtspMessage& InRequest);

	bool ParseUrl(const FString& InUrl, FString& OutHost, int32& OutPort, FString& OutPath);

	void TeardownConnection();
	int32 HandleRunError();
	
	FRtspClientConfiguration Configuration;
	
	FString Host;
	int32 Port = RtspMedia::Default::Port;
	FString Path;
	FString FullUrl; // rtsp://host:port/path

	TUniquePtr<FSocket> Socket;
	TArray<uint8> ReceiveBuffer;
	TArray<uint8> ProcessBuffer;

	ERtspClientState State = ERtspClientState::Disconnected;
	mutable FCriticalSection StateLock;

	// CSeq (Command Sequence) number incremented per request
	int32 CommandId = 0;
	
	FString SessionId;
	int32 SessionTimeoutSeconds = 60;

	double KeepAliveInterval = 30.0;
	double LastKeepAlive = 0.0;

	FRtspRequestTracker RequestTracker;

	TUniquePtr<FSdpSession> SdpSession;

	TUniquePtr<FRunnableThread> Thread;
	std::atomic<bool> bStopping{false};

	// Network jitter handling
	TUniquePtr<FRtpJitterBuffer> JitterBuffer;
	TUniquePtr<FRtpJitterEstimator> JitterEstimator;
};
