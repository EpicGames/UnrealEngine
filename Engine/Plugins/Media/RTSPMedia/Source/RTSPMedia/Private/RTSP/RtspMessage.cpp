// Copyright Epic Games, Inc. All Rights Reserved.

#include "RTSP/RtspMessage.h"

#include "RtspMediaConstants.h"

#include "CoreMinimal.h"

FRtspMessage FRtspMessage::Request(const ERtspRequestMethod InMethod, const FString& InUrl, const int32 InCommandId)
{
	FRtspMessage Request;
	Request.SetMessageType(ERtspMessageType::Request);
	Request.SetRequestMethod(InMethod);
	Request.SetRequestUrl(InUrl);
	Request.SetCommandId(InCommandId);
	return Request;
}

FRtspMessage FRtspMessage::Response(const int32 InStatusCode, const FString& InReason, int32 InCommandId)
{
	FRtspMessage Response;
	Response.SetMessageType(ERtspMessageType::Response);
	Response.SetResponseStatusCode(InStatusCode);
	Response.SetResponseReason(InReason);
	Response.SetCommandId(InCommandId);
	return Response;
}

ERtspMessageParseResult FRtspMessage::Parse(TArrayView<const uint8> InBuffer, FRtspMessage& OutMessage, int32& OutBytesConsumed)
{
	OutMessage = FRtspMessage();
	OutBytesConsumed = 0;
	
	if (InBuffer.Num() < 4)
	{
		UE_LOGF(LogRtspMedia, VeryVerbose, "RTSP message parsing attempted with a buffer containing less than 4 bytes");
		return ERtspMessageParseResult::Incomplete;
	}
	
	// Find the end of the header section which should always end with '\r\n\r\n'
	int32 HeadersEndIndex = INDEX_NONE;
	constexpr int32 HeadersEndPatternLength = 4;
	constexpr uint8 HeadersEndPattern[] = {'\r', '\n', '\r', '\n'};
	const int32 LastSearchIndex = InBuffer.Num() - HeadersEndPatternLength;
	for (int32 i = 0; i <= LastSearchIndex; ++i)
	{
		if (FMemory::Memcmp(InBuffer.GetData() + i, HeadersEndPattern, HeadersEndPatternLength) == 0)
		{
			HeadersEndIndex = i;
			break;
		}
	}

	if (HeadersEndIndex == INDEX_NONE)
	{
		UE_LOGF(LogRtspMedia, VeryVerbose, "RTSP message parsing didn't find the header end pattern.");
		return ERtspMessageParseResult::Incomplete;
	}

	const auto HeadersStringConversion = StringCast<TCHAR>(reinterpret_cast<const UTF8CHAR*>(InBuffer.GetData()), HeadersEndIndex);
	const FString HeadersString(HeadersStringConversion.Length(), HeadersStringConversion.Get());

	TArray<FString> HeaderLines;
	HeadersString.ParseIntoArrayLines(HeaderLines);

	if (HeaderLines.IsEmpty())
	{
		UE_LOGF(LogRtspMedia, Error, "RTSP message header contains zero lines");
		return ERtspMessageParseResult::Error;
	}

	if (!ParseStartLine(HeaderLines[0], OutMessage))
	{
		UE_LOGF(LogRtspMedia, Error, "RTSP message start line parsing failed");
		return ERtspMessageParseResult::Error;
	}
	
	for (int32 i = 1; i < HeaderLines.Num(); ++i)
	{
		ParseHeaderLine(HeaderLines[i], OutMessage);
	}

	const int32 BodyStartIndex = HeadersEndIndex + HeadersEndPatternLength;
	const int32 BodyLength = OutMessage.GetContentLength();

	if (BodyLength < 0)
	{
		return ERtspMessageParseResult::Error;
	}
	
	if (InBuffer.Num() - BodyStartIndex < BodyLength)
	{
		return ERtspMessageParseResult::Incomplete;
	}

	// Get body
	if (BodyLength > 0)
	{
		const auto BodyStringConversion = StringCast<TCHAR>(reinterpret_cast<const UTF8CHAR*>(InBuffer.GetData() + BodyStartIndex), BodyLength);
		OutMessage.Body = FString(BodyStringConversion.Length(), BodyStringConversion.Get());
	}

	OutBytesConsumed = BodyStartIndex + BodyLength;
	return ERtspMessageParseResult::Complete;
}

ERtspMessageType FRtspMessage::GetMessageType() const
{
	return MessageType;
}

TOptional<int32> FRtspMessage::GetCommandId() const
{
	TOptional<FString> Value = GetHeaderValue(TEXT("CSeq"));
	if (Value.IsSet())
	{
		const FString StringCommandId = Value.GetValue();
		if (StringCommandId.IsNumeric())
		{
			return FCString::Atoi(*StringCommandId);
		}
	}
	return {};
}

TOptional<ERtspRequestMethod> FRtspMessage::GetRequestMethod() const
{
	return RequestMethod;
}

TOptional<FString> FRtspMessage::GetRequestUrl() const
{
	return RequestUrl;
}

TOptional<int32> FRtspMessage::GetResponseStatusCode() const
{
	return ResponseStatusCode;
}

TOptional<FString> FRtspMessage::GetResponseReason() const
{
	return ResponseReason;
}

TOptional<FString> FRtspMessage::GetContentBaseUrl() const
{
	return GetHeaderValue(TEXT("Content-Base"));
}

int32 FRtspMessage::GetContentLength() const
{
	TOptional<FString> ContentLengthHeader = GetHeaderValue(TEXT("Content-Length"));
	
	if (ContentLengthHeader.IsSet())
	{
		const FString ContentLengthString = ContentLengthHeader.GetValue();
		if (!ContentLengthString.IsNumeric())
		{
			UE_LOGF(LogRtspMedia, Warning, "RTSP message content length header value must be numeric. Was: %ls", *ContentLengthString);
			return -1;
		}
		const int32 ContentLength = FCString::Atoi(*ContentLengthString);
		if (ContentLength >= 0)
		{
			return ContentLength;
		}
		UE_LOGF(LogRtspMedia, Warning, "Invalid Content-Length value found within RTSP message header: %d", ContentLength);
		return -1;
	}
	
	return 0;
}

TOptional<FString> FRtspMessage::GetHeaderValue(const FString& InName) const
{
	if (const FString* Value = Headers.Find(InName))
	{
		return *Value;
	}
	return {};
}

TOptional<FString> FRtspMessage::GetSessionId() const
{
	TOptional<FString> SessionHeader = GetHeaderValue(TEXT("Session"));
	if (!SessionHeader.IsSet())
	{
		return {};
	}

	// Session header format is 'sessionid;timeout={number}'
	const int32 SemicolonIndex = SessionHeader->Find(TEXT(";"));
	FString SessionIdString;
	if (SemicolonIndex != INDEX_NONE)
	{
		SessionIdString = SessionHeader->Left(SemicolonIndex).TrimStartAndEnd();
	}
	// If there is only one value in the header (i.e. no ';' delimiter)
	// then return the complete header.
	else
	{
		SessionIdString = SessionHeader.GetValue();
	}
	
	return SessionIdString.TrimStartAndEnd();
}

TOptional<int32> FRtspMessage::GetSessionTimeout() const
{
	TOptional<FString> SessionHeader = GetHeaderValue(TEXT("Session"));
	if (!SessionHeader.IsSet())
	{
		return {};
	}

	const FString TimeoutPrefix = TEXT("timeout=");
	const int32 TimeoutIndex = SessionHeader->Find(TimeoutPrefix);
	if (TimeoutIndex == INDEX_NONE)
	{
		return {};
	}

	const FString TimeoutInclusiveSuffix = SessionHeader->Mid(TimeoutIndex + TimeoutPrefix.Len());
	const int32 PostTimeoutSemiColonIndex = TimeoutInclusiveSuffix.Find(TEXT(";"));

	FString TimeoutString;
	if (PostTimeoutSemiColonIndex != INDEX_NONE)
	{
		TimeoutString = TimeoutInclusiveSuffix.Left(PostTimeoutSemiColonIndex);
	}
	else
	{
		// If this is last value in the 'Session' header then return the remaining string. 
		TimeoutString = TimeoutInclusiveSuffix;
	}

	if (TimeoutString.IsNumeric())
	{
		return FCString::Atoi(*TimeoutString);
	}

	return {};
}

const FString& FRtspMessage::GetBody() const
{
	return Body;
}

void FRtspMessage::SetMessageType(const ERtspMessageType InMessageType)
{
	MessageType = InMessageType;
}

void FRtspMessage::SetCommandId(const int32 InCommandId)
{
	SetHeader(TEXT("CSeq"), FString::FromInt(InCommandId));
}

void FRtspMessage::SetHeader(const FString& InName, const FString& InValue)
{
	Headers.Emplace(InName, InValue);
}

void FRtspMessage::SetSession(const FString& InSessionId)
{
	SetHeader(TEXT("Session"), InSessionId);
}

void FRtspMessage::SetTransport(const FRtspTransportConfiguration& InConfiguration)
{
	SetHeader(TEXT("Transport"), InConfiguration.BuildHeader());
}

void FRtspMessage::SetAccept(const FString& InMimeType)
{
	SetHeader(TEXT("Accept"), InMimeType);
}

void FRtspMessage::SetBody(const FString& InBody)
{
	Body = InBody;

	// Ensure we base the Content-Length on actual bytes, not number of characters. 
	const FTCHARToUTF8 Utf8BodyString(*InBody);
	SetHeader(TEXT("Content-Length"), FString::FromInt(Utf8BodyString.Length()));
}

void FRtspMessage::SetRequestMethod(const ERtspRequestMethod InMethod)
{
	RequestMethod = InMethod;
}

void FRtspMessage::SetRequestUrl(const FString& InUrl)
{
	RequestUrl = InUrl;
}

void FRtspMessage::SetResponseStatusCode(const int32 InStatusCode)
{
	ResponseStatusCode = InStatusCode;
}

void FRtspMessage::SetResponseReason(const FString& InReason)
{
	ResponseReason = InReason;
}

FString FRtspMessage::ToString() const
{
	TArray<FString> Lines;

	if (MessageType == ERtspMessageType::Request && RequestMethod.IsSet() && RequestUrl.IsSet())
	{
		// E.g. "OPTIONS rtsp://192.168.1.50/stream RTSP/1.0"
		TArray<FString> RequestLineComponents;
		RequestLineComponents.Add(RequestMethodToString(RequestMethod.GetValue()));
		RequestLineComponents.Add(RequestUrl.GetValue());
		RequestLineComponents.Add(TEXT("RTSP/1.0"));
		Lines.Add(FString::Join(RequestLineComponents, TEXT(" ")));
	}
	else if (MessageType == ERtspMessageType::Response && ResponseStatusCode.IsSet() && ResponseReason.IsSet())
	{
		// E.g. "RTSP/1.0 200 OK"
		TArray<FString> ResponseLineComponents;
		ResponseLineComponents.Add(TEXT("RTSP/1.0"));
		ResponseLineComponents.Add(FString::Printf(TEXT("%d"), ResponseStatusCode.GetValue()));
		ResponseLineComponents.Add(ResponseReason.GetValue());
		Lines.Add(FString::Join(ResponseLineComponents, TEXT(" ")));
	}
	else
	{
		return FString();
	}

	// Headers
	for (const TPair<FString, FString>& Pair : Headers)
	{
		Lines.Add(FString::Printf(TEXT("%s: %s"), *Pair.Key, *Pair.Value));
	}

	// The RTSP standard requires carriage return-line feed between each line and a final CR/LF at the end of the header
	return FString::Join(Lines, TEXT("\r\n")) + TEXT("\r\n\r\n") + Body;
}

FString FRtspMessage::RequestMethodToString(const ERtspRequestMethod InMethod)
{
	switch (InMethod)
	{
		case ERtspRequestMethod::Options: return TEXT("OPTIONS");
		case ERtspRequestMethod::Describe: return TEXT("DESCRIBE");
		case ERtspRequestMethod::Announce: return TEXT("ANNOUNCE");
		case ERtspRequestMethod::Setup: return TEXT("SETUP");
		case ERtspRequestMethod::Play: return TEXT("PLAY");
		case ERtspRequestMethod::Pause: return TEXT("PAUSE");
		case ERtspRequestMethod::Teardown: return TEXT("TEARDOWN");
		case ERtspRequestMethod::GetParameter: return TEXT("GET_PARAMETER");
		case ERtspRequestMethod::SetParameter: return TEXT("SET_PARAMETER");
		case ERtspRequestMethod::Redirect: return TEXT("REDIRECT");
		case ERtspRequestMethod::Record: return TEXT("RECORD");
		case ERtspRequestMethod::Unknown:
		default:
		return TEXT("UNKNOWN");
	}
}

ERtspRequestMethod FRtspMessage::StringToRequestMethod(const FString& InString)
{
	if (InString == TEXT("OPTIONS")) { return ERtspRequestMethod::Options; }
	if (InString == TEXT("DESCRIBE")) { return ERtspRequestMethod::Describe; }
	if (InString == TEXT("ANNOUNCE")) { return ERtspRequestMethod::Announce; }
	if (InString == TEXT("SETUP")) { return ERtspRequestMethod::Setup; }
	if (InString == TEXT("PLAY")) { return ERtspRequestMethod::Play; }
	if (InString == TEXT("PAUSE")) { return ERtspRequestMethod::Pause; }
	if (InString == TEXT("TEARDOWN")) { return ERtspRequestMethod::Teardown; }
	if (InString == TEXT("GET_PARAMETER")) { return ERtspRequestMethod::GetParameter; }
	if (InString == TEXT("SET_PARAMETER")) { return ERtspRequestMethod::SetParameter; }
	if (InString == TEXT("REDIRECT")) { return ERtspRequestMethod::Redirect; }
	if (InString == TEXT("RECORD")) { return ERtspRequestMethod::Record; }
	return ERtspRequestMethod::Unknown;
}

bool FRtspMessage::ParseStartLine(const FString& InLine, FRtspMessage& OutMessage)
{
    // Responses start with RTSP/<version> and contain a status code and reason.
    // E.g. "RTSP/1.0 404 Not Found"
    // The reason string can contain spaces so we can't tokenise on " " for responses.
    if (InLine.StartsWith(TEXT("RTSP/")))
    {
        OutMessage.MessageType = ERtspMessageType::Response;

        const int32 FirstSpaceIndex = InLine.Find(TEXT(" "));
        if (FirstSpaceIndex == INDEX_NONE)
        {
            UE_LOGF(LogRtspMedia, Warning, "Malformed RTSP response start line, no space characters found: \"%ls\"", *InLine);
            return false;
        }

        const int32 SecondSpaceIndex = InLine.Find(TEXT(" "), ESearchCase::IgnoreCase, ESearchDir::FromStart, FirstSpaceIndex + 1);
        if (SecondSpaceIndex == INDEX_NONE)
        {
            UE_LOGF(LogRtspMedia, Warning, "Malformed RTSP response start line, second space character not found: \"%ls\"", *InLine);
            return false;
        }

        const FString StatusCodeString = InLine.Mid(FirstSpaceIndex + 1, SecondSpaceIndex - FirstSpaceIndex - 1);
        OutMessage.ResponseStatusCode = FCString::Atoi(*StatusCodeString);
        // The rest of the line contains the reason (e.g 'Not Found')
        OutMessage.ResponseReason = InLine.Mid(SecondSpaceIndex + 1);
    }
    // Requests start with a METHOD, URL and RTSP/1.0
    // E.g. "OPTIONS rtsp://<address>/<path> RTSP/1.0"
    // The only spaces in a request status line are between the elements,
    // so in this case we can tokenise on whitespace.
    else
    {
        OutMessage.MessageType = ERtspMessageType::Request;

        TArray<FString> RequestStatusLineComponents;
        InLine.ParseIntoArray(RequestStatusLineComponents, TEXT(" "));

        if (RequestStatusLineComponents.Num() != 3)
        {
            UE_LOGF(LogRtspMedia, Warning, "Expected 3 components in RTSP request status line \"%ls\" but found %d.", *InLine, RequestStatusLineComponents.Num());
            return false;
        }

        OutMessage.RequestMethod = StringToRequestMethod(RequestStatusLineComponents[0]);
    	OutMessage.RequestUrl = RequestStatusLineComponents[1];
    }

    return true;
}

void FRtspMessage::ParseHeaderLine(const FString& Line, FRtspMessage& OutMessage)
{
	// E.g. 'HeaderName: HeaderValue'
	// Could be 'HeaderName:HeaderValue'
	const int32 ColonIndex = Line.Find(TEXT(":"));
	if (ColonIndex != INDEX_NONE)
	{
		// Account for non-compliant whitespace around the colon by trimming
		const FString HeaderName = Line.Left(ColonIndex).TrimStartAndEnd();
		const FString HeaderValue = Line.Mid(ColonIndex + 1).TrimStartAndEnd();
		OutMessage.SetHeader(HeaderName, HeaderValue);
	}
}
