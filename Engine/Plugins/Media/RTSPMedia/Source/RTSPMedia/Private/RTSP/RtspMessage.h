// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "RtspTransportConfiguration.h"

/**
 * We need to handle the possibility that incoming message header names are using the incorrect case (E.g. 'cseq' instead of 'CSeq').
 * This custom KeyFuncs implementation will allow us to match names in a case-insensitive fashion.
 */
struct FCaseInsensitiveMapKeyMatch : TDefaultMapHashableKeyFuncs<FString, FString, false>
{
	static FORCEINLINE bool Matches(const FString& InA, const FString& InB)
	{
		return InA.Equals(InB, ESearchCase::IgnoreCase);
	}

	static FORCEINLINE uint32 GetKeyHash(const FString& InKey)
	{
		return GetTypeHash(InKey.ToLower());
	}
};

enum class ERtspMessageType
{
	Unknown,
	Request,
	Response
};

enum class ERtspRequestMethod
{
	Unknown,
	Options,
	Describe,
	Announce,
	Setup,
	Play,
	Pause,
	Teardown,
	GetParameter,
	SetParameter,
	Redirect,
	Record
};

enum class ERtspMessageParseResult : uint8
{
	Incomplete,
	Complete,
	Error
};

class FRtspMessage
{
public:
	static FRtspMessage Request(const ERtspRequestMethod InMethod, const FString& InUrl, const int32 InCommandId);
	static FRtspMessage Response(const int32 InStatusCode, const FString& InReason, const int32 InCommandId);
	static ERtspMessageParseResult Parse(TArrayView<const uint8> InData, FRtspMessage& OutMessage, int32& OutBytesConsumed);
	
	static FString RequestMethodToString(const ERtspRequestMethod InMethod);
	static ERtspRequestMethod StringToRequestMethod(const FString& InString);
	
	ERtspMessageType GetMessageType() const;

	TOptional<int32> GetCommandId() const;
	
	TOptional<ERtspRequestMethod> GetRequestMethod() const;
	TOptional<FString> GetRequestUrl() const;
	TOptional<int32> GetResponseStatusCode() const;
	TOptional<FString>  GetResponseReason() const;
	TOptional<FString> GetContentBaseUrl() const;
	int32 GetContentLength() const;
	TOptional<FString> GetHeaderValue(const FString& InName) const;
	TOptional<FString> GetSessionId() const;
	TOptional<int32> GetSessionTimeout() const;
	const FString& GetBody() const;

	void SetMessageType(const ERtspMessageType InMessageType);
	void SetCommandId(const int32 InCommandId);
	void SetHeader(const FString& InName, const FString& InValue);
	void SetSession(const FString& InSessionId);
	void SetTransport(const FRtspTransportConfiguration& InConfiguration);
	void SetAccept(const FString& InMimeType);
	void SetBody(const FString& InBody);

	void SetRequestMethod(const ERtspRequestMethod InMethod);
	void SetRequestUrl(const FString& InUrl);
	void SetResponseStatusCode(const int32 InStatusCode);
	void SetResponseReason(const FString& InReason);

	FString ToString() const;

private:
	static bool ParseStartLine(const FString& InLine, FRtspMessage& OutMessage);
	static void ParseHeaderLine(const FString& InLine, FRtspMessage& OutMessage);
	
	ERtspMessageType MessageType = ERtspMessageType::Unknown;
	
	// Requests
	TOptional<ERtspRequestMethod> RequestMethod;
	TOptional<FString> RequestUrl;

	// Responses
	TOptional<int32> ResponseStatusCode;
	TOptional<FString> ResponseReason;
	
	TMap<FString, FString, FDefaultSetAllocator, FCaseInsensitiveMapKeyMatch> Headers;
	
	FString Body;
};
