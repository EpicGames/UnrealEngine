// Copyright Epic Games, Inc. All Rights Reserved.

#include "RtspRequestTracker.h"

#include "HAL/PlatformTime.h"

void FRtspRequestTracker::AddPendingRequest(int32 InCommandId, const FRtspPendingRequest& InRequest)
{
	PendingRequests.Emplace(InCommandId, InRequest);
}

bool FRtspRequestTracker::HasPendingRequests() const
{
	return !PendingRequests.IsEmpty();
}

bool FRtspRequestTracker::TakePendingRequest(const int32 CommandId, FRtspPendingRequest& OutRequest)
{
	return PendingRequests.RemoveAndCopyValue(CommandId, OutRequest); 
}

TArray<FRtspPendingRequest> FRtspRequestTracker::GetExpiredRequests(const double TimeoutSeconds)
{
	const double Now = FPlatformTime::Seconds();
	TArray<FRtspPendingRequest> ExpiredRequests;
	auto Iterator = PendingRequests.CreateIterator();
	while (Iterator)
	{
		FRtspPendingRequest& PendingRequest = Iterator.Value();
		if (Now - PendingRequest.SentTimestamp > TimeoutSeconds)
		{
			ExpiredRequests.Emplace(MoveTemp(PendingRequest));
			Iterator.RemoveCurrent();
		}
		++Iterator;
	}
	return ExpiredRequests;
}

void FRtspRequestTracker::Clear()
{
	PendingRequests.Empty();
}
