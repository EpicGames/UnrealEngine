// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "RtspMessage.h"

struct FRtspPendingRequest
{
	FRtspMessage Request;
	double SentTimestamp = 0.0;
};

class FRtspRequestTracker
{
public:
	void AddPendingRequest(int32 InCommandId, const FRtspPendingRequest& InRequest);
	bool HasPendingRequests() const;
	bool TakePendingRequest(const int32 CommandId, FRtspPendingRequest& OutRequest);
	TArray<FRtspPendingRequest> GetExpiredRequests(const double TimeoutSeconds);
	void Clear();
private:
	TMap<int32, FRtspPendingRequest> PendingRequests;
};
