// Copyright Epic Games, Inc. All Rights Reserved.

#include "RTSP/RtspTransportConfiguration.h"

FRtspTransportConfiguration FRtspTransportConfiguration::TcpInterleaved(int32 InRtpChannel, int32 InRtcpChannel)
{
	FRtspTransportConfiguration Configuration;
	Configuration.Protocol = TransportProtocol::TCP;
	Configuration.TcpInterleavedRtpChannel = InRtpChannel;
	Configuration.TcpInterleavedRtcpChannel = InRtcpChannel;
	return Configuration;
}

FRtspTransportConfiguration FRtspTransportConfiguration::Udp(int32 InClientRtpPort, int32 InClientRtcpPort)
{
	FRtspTransportConfiguration Configuration;
	Configuration.Protocol = TransportProtocol::UDP;
	Configuration.UdpClientRtpPort = InClientRtpPort;
	Configuration.UdpClientRtcpPort = InClientRtcpPort;
	return Configuration;
}

FString FRtspTransportConfiguration::BuildHeader() const
{
	TArray<FString> StackComponents;
	StackComponents.Add(TEXT("RTP"));
	StackComponents.Add(TEXT("AVP"));

	FString ParameterKey;
	FString ParameterValue;
	
	if (Protocol == TransportProtocol::TCP)
	{
		StackComponents.Add(TEXT("TCP"));
		ParameterKey = TEXT("interleaved");
		ParameterValue = FString::Printf(TEXT("%d-%d"), TcpInterleavedRtpChannel, TcpInterleavedRtcpChannel);
	}
	else if (Protocol == TransportProtocol::UDP)
	{
		ParameterKey = TEXT("client_port");
		ParameterValue = FString::Printf(TEXT("%d-%d"), UdpClientRtpPort, UdpClientRtcpPort);
	}

	TArray<FString> Components;
	Components.Add(FString::Join(StackComponents, TEXT("/")));
	Components.Add(TEXT("unicast"));
	Components.Add(ParameterKey + TEXT("=") + ParameterValue);

	return FString::Join(Components, TEXT(";"));
}
