// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

struct FRtspTransportConfiguration
{
	enum class TransportProtocol
	{
		TCP,
		UDP
	};
	
	TransportProtocol Protocol = TransportProtocol::TCP;

	int32 TcpInterleavedRtpChannel = 0;
	int32 TcpInterleavedRtcpChannel = 1;

	int32 UdpClientRtpPort = 0;
	int32 UdpClientRtcpPort = 0;

	static FRtspTransportConfiguration TcpInterleaved(int32 InRtpChannel, int32 InRtcpChannel);
	static FRtspTransportConfiguration Udp(int32 InClientRtpPort, int32 InClientRtcpPort);

	FString BuildHeader() const;
};
