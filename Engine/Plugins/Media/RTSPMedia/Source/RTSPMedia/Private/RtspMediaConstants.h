// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"
#include "Misc/Guid.h"
#include "UObject/NameTypes.h"

DECLARE_LOG_CATEGORY_EXTERN(LogRtspMedia, Log, All);

namespace RtspMedia
{
	inline const FName PlayerName(TEXT("RTSPMediaPlayer"));
	inline constexpr FGuid PlayerPluginGUID(0x690B0293, 0x4C784EBF, 0xAF4769FF, 0x104000BD);

	namespace Option
	{
		inline const FName AutoReconnect("AutoReconnect");
		inline const FName ProvideCpuBuffer("ProvideCpuBuffer");
		inline const FName DecoderBufferSize("DecoderBufferSize");
		inline const FName DecoderPollIntervalMs("DecoderPollIntervalMs");
		inline const FName JitterBufferAutoAdjust("JitterBufferAutoAdjust");
		inline const FName JitterBufferDepthMs("JitterBufferDepthMs");
		inline const FName JitterBufferObservationWindowSeconds("JitterBufferObservationWindowSeconds");
		inline const FName MaxFragmentBufferSize("MaxFragmentBufferSize");
		inline const FName MaxQueuedVideoSamples("MaxQueuedVideoSamples");
		inline const FName MaxReconnectAttempts("MaxReconnectAttempts");
		inline const FName MaxReconnectDelaySeconds("MaxReconnectDelaySeconds");
		inline const FName MinReconnectDelaySeconds("MinReconnectDelaySeconds");
		inline const FName RequestTimeoutSeconds("RequestTimeoutSeconds");
		inline const FName SocketBufferSize("SocketBufferSize");
		inline const FName TransportProtocol("TransportProtocol");
	}
	
}
