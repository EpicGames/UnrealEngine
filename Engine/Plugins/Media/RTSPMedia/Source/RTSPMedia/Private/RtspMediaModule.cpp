// Copyright Epic Games, Inc. All Rights Reserved.

#include "Player/RtspMediaPlayer.h"
#include "RtspMediaConstants.h"

#include "IMediaModule.h"
#include "IMediaPlayerFactory.h"
#include "IMediaPlayer.h"
#include "Internationalization/Internationalization.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

DEFINE_LOG_CATEGORY(LogRtspMedia);

#define LOCTEXT_NAMESPACE "RtspMediaModule"

class FRtspMediaModule : public IModuleInterface, public IMediaPlayerFactory
{
public:
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	//~ Begin IMediaPlayerFactory
	virtual bool CanPlayUrl(const FString& Url, const IMediaOptions* Options, TArray<FText>* OutWarnings, TArray<FText>* OutErrors) const override;
	virtual int32 GetPlayabilityConfidenceScore(const FString& Url, const IMediaOptions* Options, TArray<FText>* OutWarnings, TArray<FText>* OutErrors) const override;
	virtual TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CreatePlayer(IMediaEventSink& InEventSink) override;
	virtual FText GetDisplayName() const override;
	virtual FName GetPlayerName() const override;
	virtual FGuid GetPlayerPluginGUID() const override;
	virtual const TArray<FString>& GetSupportedPlatforms() const override;
	virtual bool SupportsFeature(EMediaFeature Feature) const override;
	//~ End IMediaPlayerFactory

private:
	/** List of platforms that the media player support. */
	TArray<FString> SupportedPlatforms;
};

void FRtspMediaModule::StartupModule()
{
	// supported platforms
	SupportedPlatforms.Add(TEXT("Windows"));
	SupportedPlatforms.Add(TEXT("Mac"));
	SupportedPlatforms.Add(TEXT("Linux"));

	// register player factory
	if (IMediaModule* MediaModule = FModuleManager::LoadModulePtr<IMediaModule>("Media"))
	{
		MediaModule->RegisterPlayerFactory(*this);
	}
}

void FRtspMediaModule::ShutdownModule()
{
	// unregister player factory
	if (IMediaModule* MediaModule = FModuleManager::GetModulePtr<IMediaModule>("Media"))
	{
		MediaModule->UnregisterPlayerFactory(*this);
	}
}

bool FRtspMediaModule::CanPlayUrl(const FString& Url, const IMediaOptions* Options, TArray<FText>* OutWarnings, TArray<FText>* OutErrors) const
{
	return GetPlayabilityConfidenceScore(Url, Options, OutWarnings, OutErrors) > 0;
}

int32 FRtspMediaModule::GetPlayabilityConfidenceScore(const FString& Url, const IMediaOptions* Options, TArray<FText>* OutWarnings, TArray<FText>* OutErrors) const
{
	FString Scheme;
	FString Location;

	// get scheme
	if (!Url.Split(TEXT("://"), &Scheme, &Location, ESearchCase::CaseSensitive))
	{
		if (OutErrors != nullptr)
		{
			OutErrors->Add(LOCTEXT("NoSchemeFound", "No URI scheme found"));
		}

		return 0;
	}

	if (!Scheme.Equals(TEXT("rtsp"), ESearchCase::IgnoreCase))
	{
		if (OutErrors != nullptr)
		{
			OutErrors->Add(FText::Format(LOCTEXT("SchemeNotSupported", "The URI scheme '{0}' is not supported"), FText::FromString(Scheme)));
		}

		return 0;
	}

	// Returning a score less than 80 in order to avoid taking priority over the
	// native Android and Windows (WMF) implementations (which return 80).
	// Once we are confident that this module provides superior
	// functionality we will consider increasing this score.
	// This only impacts when a StreamMediaSource is used, for our
	// dedicated RTSPMediaSource our own player implementation will
	// always be used.
	return 70;
}

TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> FRtspMediaModule::CreatePlayer(IMediaEventSink& InEventSink) 
{
	return MakeShared<FRtspMediaPlayer, ESPMode::ThreadSafe>(InEventSink);
}

FText FRtspMediaModule::GetDisplayName() const
{
	return LOCTEXT("DisplayName", "RTSP Media");
}

FName FRtspMediaModule::GetPlayerName() const
{
	return RtspMedia::PlayerName;
}

FGuid FRtspMediaModule::GetPlayerPluginGUID() const
{
	return RtspMedia::PlayerPluginGUID;
}

const TArray<FString>& FRtspMediaModule::GetSupportedPlatforms() const
{
	return SupportedPlatforms;
}

bool FRtspMediaModule::SupportsFeature(EMediaFeature Feature) const
{
	return Feature == EMediaFeature::VideoSamples;
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FRtspMediaModule, RTSPMedia)
