// Copyright Epic Games, Inc. All Rights Reserved.

#include "RtspMediaSource.h"

#include "RtspMediaConstants.h"

FString URtspMediaSource::GetUrl() const
{
	const FString TrimmedHost = Host.TrimStartAndEnd();
	const FString TrimmedPath = Path.TrimStartAndEnd();

	FString Url = FString::Printf(TEXT("rtsp://%s:%d"), *TrimmedHost, Port);
	if (!TrimmedPath.IsEmpty())
	{
		Url += "/" + TrimmedPath;
	}
	return Url;
}

bool URtspMediaSource::Validate() const
{
	const FString TrimmedHost = Host.TrimStartAndEnd();
	const FString TrimmedPath = Path.TrimStartAndEnd();
	
	if (TrimmedHost.IsEmpty() ||
		Port <= 0 ||
		Port > TNumericLimits<uint16>::Max() ||
		TrimmedPath.StartsWith(TEXT("/")))
	{
		return false;
	}
	
	return true;
}

bool URtspMediaSource::GetMediaOption(const FName& InKey, bool InDefaultValue) const
{
	if (InKey == RtspMedia::Option::AutoReconnect)
	{
		return bAutoReconnect;
	}
	
	if (InKey == RtspMedia::Option::ProvideCpuBuffer)
	{
		return bProvideCpuBuffer;
	}

	if (InKey == RtspMedia::Option::JitterBufferAutoAdjust)
	{
		return bJitterBufferAutoAdjust;
	}

	return Super::GetMediaOption(InKey, InDefaultValue);
}

FName URtspMediaSource::GetDesiredPlayerName() const
{
	return RtspMedia::PlayerName;
}

int64 URtspMediaSource::GetMediaOption(const FName& InKey, int64 InDefaultValue) const
{
	if (InKey == RtspMedia::Option::DecoderBufferSize)
	{
		return DecoderBufferSize;
	}

	if (InKey == RtspMedia::Option::DecoderPollIntervalMs)
	{
		return DecoderPollIntervalMs;
	}

	if (InKey == RtspMedia::Option::JitterBufferDepthMs)
	{
		return JitterBufferDepthMs;
	}
	
	if (InKey == RtspMedia::Option::MaxQueuedVideoSamples)
	{
		return MaxQueuedVideoSamples;
	}

	if (InKey == RtspMedia::Option::MaxFragmentBufferSize)
	{
		return MaxFragmentBufferSizeMb;
	}

	if (InKey == RtspMedia::Option::MaxReconnectAttempts)
	{
		return MaxReconnectAttempts;
	}

	if (InKey == RtspMedia::Option::TransportProtocol)
	{
		return static_cast<int64>(TransportProtocol);
	}

	if (InKey == RtspMedia::Option::SocketBufferSize)
	{
		return static_cast<int64>(SocketBufferSizeKb);
	}

	return Super::GetMediaOption(InKey, InDefaultValue);
}

double URtspMediaSource::GetMediaOption(const FName& InKey, double InDefaultValue) const
{
	if (InKey == RtspMedia::Option::JitterBufferObservationWindowSeconds)
	{
		return JitterBufferObservationWindowSeconds;
	}

	if (InKey == RtspMedia::Option::MaxReconnectDelaySeconds)
	{
		return MaxReconnectDelaySeconds;
	}
	
	if (InKey == RtspMedia::Option::MinReconnectDelaySeconds)
	{
		return MinReconnectDelaySeconds;
	}
	
	if (InKey == RtspMedia::Option::RequestTimeoutSeconds)
	{
		return RequestTimeoutSeconds;
	}
	
	return Super::GetMediaOption(InKey, InDefaultValue);
}

bool URtspMediaSource::HasMediaOption(const FName& InKey) const
{
	if (InKey == RtspMedia::Option::ProvideCpuBuffer ||
		InKey == RtspMedia::Option::AutoReconnect ||
		InKey == RtspMedia::Option::DecoderBufferSize ||
		InKey == RtspMedia::Option::DecoderPollIntervalMs ||
		InKey == RtspMedia::Option::JitterBufferAutoAdjust ||
		InKey == RtspMedia::Option::JitterBufferDepthMs ||
		InKey == RtspMedia::Option::JitterBufferObservationWindowSeconds ||
		InKey == RtspMedia::Option::MaxFragmentBufferSize ||
		InKey == RtspMedia::Option::MaxQueuedVideoSamples ||
		InKey == RtspMedia::Option::MaxReconnectAttempts ||
		InKey == RtspMedia::Option::MaxReconnectDelaySeconds ||
		InKey == RtspMedia::Option::MinReconnectDelaySeconds ||
		InKey == RtspMedia::Option::RequestTimeoutSeconds ||
		InKey == RtspMedia::Option::SocketBufferSize ||
		InKey == RtspMedia::Option::TransportProtocol)
	{
		return true;
	}

	return Super::HasMediaOption(InKey);
}
