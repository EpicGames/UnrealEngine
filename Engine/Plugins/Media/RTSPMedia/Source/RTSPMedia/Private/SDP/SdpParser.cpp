// Copyright Epic Games, Inc. All Rights Reserved.

#include "SdpParser.h"

#include "RtspMediaConstants.h"
#include "SDP/SdpSession.h"

#include "Misc/Base64.h"

TUniquePtr<FSdpSession> FSdpParser::Parse(const FString& InSdpBody, const FString& InContentBaseUrl)
{
	if (InSdpBody.IsEmpty())
	{
		UE_LOGF(LogRtspMedia, Error, "Empty SDP body");
		return nullptr;
	}

	TUniquePtr<FSdpSession> Session = MakeUnique<FSdpSession>();
	Session->ContentBaseUrl = InContentBaseUrl;

	TArray<FString> Lines;
	InSdpBody.ParseIntoArrayLines(Lines);

	TArray<FSdpMediaTrack> MediaTracks;

	for (int32 i = 0; i < Lines.Num(); ++i) 
	{
		const FString& Line = Lines[i];

		// Skip blank lines
		if (Line.IsEmpty())
		{
			continue;
		}

		// Skip malformed lines
		// SDP lines are in the following format "Type=Value"
		if (Line.Len() < 2 || Line[1] != TEXT('='))
		{
			continue;
		}

		const TCHAR Type = Line[0];
		FString Value = Line.Mid(2);

		switch (Type)
		{
			// Session Name
			case TEXT('s'):
				Session->SessionName = Value;
				break;
			// Session Info
			case TEXT('i'):
				Session->SessionInfo = Value;
				break;
			// Media line
			case TEXT('m'):
				{
					FSdpMediaTrack MediaTrack;
					if (!ParseMediaLine(Value, MediaTrack))
					{
						UE_LOGF(LogRtspMedia, Warning, "SDP parser encountered invalid media line '%ls'.", *Value);
						continue;
					}
					MediaTracks.Add(MoveTemp(MediaTrack));
					break;
				}
			case TEXT('a'):
				if (!MediaTracks.IsEmpty())
				{
					if (!ParseAttributeLine(Value, MediaTracks.Last()))
					{
						continue;
					}
				}
				else
				{
					UE_LOGF(LogRtspMedia, Verbose, "SDP parser encountered attribute line '%ls' but no media track is defined at this point.", *Value);
				}
				break;
			default:
				// Other types are ignored
				break;
		}
	}

	// Split media tracks into video and audio within the final session
	for (FSdpMediaTrack& MediaTrack : MediaTracks)
	{
		UE_LOGF(LogRtspMedia, Verbose, "Parsed SDP track: Type: %ls Codec: %ls Clock Rate: %u Control URL: %ls", *MediaTrack.MediaType, *MediaTrack.Codec, MediaTrack.ClockRate, *MediaTrack.ControlUrl);

		if (MediaTrack.MediaType == TEXT("video"))
		{
			Session->VideoTracks.Add(MoveTemp(MediaTrack));
		}
		else if (MediaTrack.MediaType == TEXT("audio"))
		{
			Session->AudioTracks.Add(MoveTemp(MediaTrack));
		}
		// There are other types of tracks which we ignore
		// - text (subtitles)
		// - application (generic metadata)
		// - message (irrelevant for RTSP)
	}

	UE_LOGF(LogRtspMedia, Verbose, "Parsed SDP: %d video tracks, %d audio tracks", Session->VideoTracks.Num(), Session->AudioTracks.Num());

	return Session;
}

bool FSdpParser::ParseMediaLine(const FString& InMediaLine, FSdpMediaTrack& OutMediaTrack)
{
	// E.g. "video 0 RTP/AVP 96"

	TArray<FString> Parts;
	InMediaLine.ParseIntoArray(Parts, TEXT(" "));

	if (Parts.Num() >= 4)
	{
		OutMediaTrack.MediaType = Parts[0];
		OutMediaTrack.Port = FCString::Atoi(*Parts[1]);
		OutMediaTrack.Protocol =  Parts[2];
		OutMediaTrack.PayloadType = FCString::Atoi(*Parts[3]);
		return true;
	}
	else
	{
		UE_LOGF(LogRtspMedia, Warning, "Ignoring malformed media line in SDP: %ls", *InMediaLine);
		return false;
	}
}

bool FSdpParser::ParseAttributeLine(const FString& InAttributeLine, FSdpMediaTrack& InMediaTrack)
{
	// E.g. "rtpmap:96 H264/90000"
	// E.g. "fmtp:96 packetization-mode=1; profile-level-id=420020; sprop-parameter-sets=J0IAIKtAWgUNNwICAgI=,KM48gA=="
	// E.g. "control:trackID=0"
	
	int32 ColonIndex;
	if (InAttributeLine.FindChar(TEXT(':'), ColonIndex))
	{
		const FString AttributeName = InAttributeLine.Left(ColonIndex).TrimStartAndEnd();
		const FString AttributeData = InAttributeLine.Mid(ColonIndex + 1).TrimStartAndEnd();

		// RTP Map
		if (AttributeName == TEXT("rtpmap"))
		{
			return ParseRtpMap(AttributeData, InMediaTrack);
		}
		// Format Parameters
		else if (AttributeName == TEXT("fmtp"))
		{
			return ParseFormatParameters(AttributeData, InMediaTrack);
		}
		else if (AttributeName == TEXT("control"))
		{
			InMediaTrack.ControlUrl = AttributeData;
		}
	}

	return true;
}

bool FSdpParser::ParseRtpMap(const FString& InRtpMap, FSdpMediaTrack& InTrack)
{
	// E.g. "96 H264/90000"
	// E.g. "94 MPEG-4-GENERIC/44100/2"

	int32 SpaceIndex;
	if (!InRtpMap.FindChar(TEXT(' '), SpaceIndex))
	{
		UE_LOGF(LogRtspMedia, Warning, "SDP track does not contain format information");
		return false;
	}

	const FString Encoding = InRtpMap.Mid(SpaceIndex + 1);

	TArray<FString> EncodingParts;
	Encoding.ParseIntoArray(EncodingParts, TEXT("/"));

	if (EncodingParts.Num() >= 1)
	{
		InTrack.Codec = EncodingParts[0];
	}
	
	if (EncodingParts.Num() >= 2)
	{
		const FString ClockRateString = EncodingParts[1];
		const int32 ParsedClockRate = FCString::Atoi(*ClockRateString);
		if (ParsedClockRate <= 0)
		{
			UE_LOGF(LogRtspMedia, Warning, "SDP parser found invalid clock rate: '%ls'", *ClockRateString);
			return false;
		}
		InTrack.ClockRate = static_cast<uint32>(ParsedClockRate);
	}
	
	if (EncodingParts.Num() >= 3)
	{
		InTrack.Channels = FCString::Atoi(*EncodingParts[2]);
	}

	return true;
}

bool FSdpParser::ParseFormatParameters(const FString& InFormatParameters, FSdpMediaTrack& InTrack)
{
	// e.g. "96 parameter1=value1;parameter2=value2"

	int32 SpaceIndex;
	if (!InFormatParameters.FindChar(TEXT(' '), SpaceIndex))
	{
		return false;
	}

	// Raw parameters
	InTrack.FormatParameters = InFormatParameters.Mid(SpaceIndex + 1);

	TArray<FString> Parameters;
	InTrack.FormatParameters.ParseIntoArray(Parameters, TEXT(";"));

	for (const FString& Parameter : Parameters)
	{
		FString ParameterKey;
		FString ParameterValue;

		if (Parameter.Split(TEXT("="), &ParameterKey, &ParameterValue))
		{
			ParameterKey.TrimStartAndEndInline();
			ParameterValue.TrimStartAndEndInline();

			// H.264 Stream Property Parameter Sets
			if (ParameterKey == TEXT("sprop-parameter-sets"))
			{
				if (!ParseH264StreamPropertyParameterSets(ParameterValue, InTrack))
				{
					return false;
				}
			}
			// H.264 packetization mode
			// 0: One NAL per RTP packet
			// 1: Non-Interleaved: Allows for:
			//   a. Aggregation (STAP-A) combining multiple NALs in a single RTP packet.
			//   b. Fragmentation (FU-A) across packets.
			// 2: Interleaved: Out-of-order transmission
			else if (ParameterKey == TEXT("packetization-mode"))
			{
				InTrack.PacketizationMode = FCString::Atoi(*ParameterValue);
			}
			// MPEG4-GENERIC payload format (includes AAC)
			else if (ParameterKey == TEXT("config"))
			{
				if (HexDecode(ParameterValue, InTrack.AudioConfig))
				{
					UE_LOGF(LogRtspMedia, Verbose, "Decoded AAC config: %d bytes", InTrack.AudioConfig.Num());
				}
				else
				{
					UE_LOGF(LogRtspMedia, Warning, "Failed to decode hex AAC configuration string: %ls", *ParameterValue);
					return false;
				}
			}
		}
	}

	return true;
}

bool FSdpParser::ParseH264StreamPropertyParameterSets(const FString& InH264StreamPropertyParameterSets, FSdpMediaTrack& InTrack)
{
	// e.g. "Z2QAH6zZQFAFuhAAAAMAEAAAAwPI8YMZYA==,aOvjyyLA"

	TArray<FString> Parts;
	InH264StreamPropertyParameterSets.ParseIntoArray(Parts, TEXT(","));

	if (Parts.Num() >= 1)
	{
		const FString SequenceParameterSetBase64String = Parts[0];
		if (Base64Decode(SequenceParameterSetBase64String, InTrack.SequenceParameterSet))
		{
			UE_LOGF(LogRtspMedia, Verbose, "Parsed sequence parameter set: %d bytes", InTrack.SequenceParameterSet.Num());
		}
		else
		{
			UE_LOGF(LogRtspMedia, Warning, "Failed to decode sequence parameter set from base 64 encoded string: %ls", *SequenceParameterSetBase64String);
			return false;
		}
	}

	if (Parts.Num() >= 2)
	{
		const FString PictureParameterSetBase64String = Parts[1];
		if (Base64Decode(PictureParameterSetBase64String, InTrack.PictureParameterSet))
		{
			UE_LOGF(LogRtspMedia, Verbose, "Parsed picture parameter set: %d bytes", InTrack.PictureParameterSet.Num());
		}
		else
		{
			UE_LOGF(LogRtspMedia, Warning, "Failed to decode picture parameter set from base 64 encoded string: %ls", *PictureParameterSetBase64String);
			return false;
		}
	}

	return true;
}

bool FSdpParser::Base64Decode(const FString& InBase64EncodedString, TArray<uint8>& OutBuffer)
{
	const FString Trimmed = InBase64EncodedString.TrimStartAndEnd();
	return FBase64::Decode(Trimmed, OutBuffer);
}

bool FSdpParser::HexDecode(const FString& InHexEncodedString, TArray<uint8>& OutBuffer)
{
	TArray<uint8> Decoded;
	const FString Trimmed = InHexEncodedString.TrimStartAndEnd();

	// Check we have an even number of characters for hex decoding
	if (Trimmed.Len() % 2 != 0)
	{
		return false;
	}
	
	for (int32 i = 0; i + 1 < Trimmed.Len(); i += 2)
	{
		FString HexString = Trimmed.Mid(i, 2);
		if (!FChar::IsHexDigit(HexString[0]) || !FChar::IsHexDigit(HexString[1]))
		{
			return false;
		}
		uint8 ByteValue = static_cast<uint8>(FCString::Strtoi(*HexString, nullptr, 16));
		Decoded.Add(ByteValue);
	}
	OutBuffer = Decoded;
	return true;
}
