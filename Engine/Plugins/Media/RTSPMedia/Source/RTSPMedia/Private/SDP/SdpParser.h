// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

struct FSdpMediaTrack;
struct FSdpSession;

class FSdpParser
{
public:
	static TUniquePtr<FSdpSession> Parse(const FString& InSdpBody, const FString& InContentBaseUrl);

private:
	static bool ParseMediaLine(const FString& InMediaLine, FSdpMediaTrack& OutMediaTrack);
	static bool ParseAttributeLine(const FString& InAttributeLine, FSdpMediaTrack& InMediaTrack);
	static bool ParseRtpMap(const FString& InRtpMap, FSdpMediaTrack& InTrack);
	static bool ParseFormatParameters(const FString& InFormatParameters, FSdpMediaTrack& InTrack);
	static bool ParseH264StreamPropertyParameterSets(const FString& InH264StreamPropertyParameterSets, FSdpMediaTrack& InTrack);

	static bool Base64Decode(const FString& InBase64EncodedString, TArray<uint8>& OutBuffer);
	static bool HexDecode(const FString& InHexEncodedString, TArray<uint8>& OutBuffer);
};
