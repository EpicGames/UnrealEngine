// Copyright Epic Games, Inc. All Rights Reserved.

#include "SdpSession.h"

bool FSdpSession::HasVideo() const
{
	return !VideoTracks.IsEmpty();
}

bool FSdpSession::HasAudio() const
{
	return !AudioTracks.IsEmpty();
}

const FSdpMediaTrack* FSdpSession::GetFirstVideoTrack() const
{
	return HasVideo() ? &VideoTracks[0] : nullptr;
}

const FSdpMediaTrack* FSdpSession::GetFirstAudioTrack() const
{
	return HasAudio() ? &AudioTracks[0] : nullptr;
}

FString FSdpSession::ResolveControlUrl(const FString& InControlUrl) const
{
	// If the control URL is '*' this is an aggregate control URL which means we use the base url to control all streams
	// Control URLs can be specified at a session or track level.
	if (InControlUrl == TEXT("*"))
	{
		return ContentBaseUrl;
	}

	const FString SchemeAuthorityDelimiter = TEXT("://");
	
	// Any URL containing '://' should be preceded by a schema and is therefore an absolute URL
	if (InControlUrl.Contains(SchemeAuthorityDelimiter))
	{
		return InControlUrl;
	}

	// If a control url starts with '/' it's an absolute path which should be appended
	// to the root server URL e.g. rtsp://example.com:8554/InControlUrl.
	// As opposed to being appended to the end of an existing path within the content base URL
	if (InControlUrl.StartsWith(TEXT("/")))
	{
		// Find the server root URL from the content base URL
		const int32 SchemeEndIndex = ContentBaseUrl.Find(SchemeAuthorityDelimiter);
		if (SchemeEndIndex != INDEX_NONE)
		{
			const int32 PathStart = ContentBaseUrl.Find(
				TEXT("/"),
				ESearchCase::IgnoreCase,
				ESearchDir::FromStart,
				SchemeEndIndex + SchemeAuthorityDelimiter.Len());

			FString Root;
			if (PathStart != INDEX_NONE)
			{
				Root = ContentBaseUrl.Left(PathStart);
			}
			else // The content base url doesn't contain a path
			{
				Root = ContentBaseUrl;
			}

			return Root + InControlUrl;
		}
	}

	// Otherwise the control "url" (which should be a simple path at this stage) is appended to the control base URL
	FString BaseUrl;
	// Strip the trailing '/' character from the base URL
	if (ContentBaseUrl.EndsWith(TEXT("/")))
	{
		BaseUrl = ContentBaseUrl.Left(ContentBaseUrl.Len() - 1);
	}
	else
	{
		BaseUrl = ContentBaseUrl;
	}

	return BaseUrl + TEXT("/") + InControlUrl;
}
