// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

struct FSdpMediaTrack
{
	// Media Description
	// "m=video 0 RTP/AVP 96"
	FString MediaType;
	int32 Port = 0;
	FString Protocol;
	int32 PayloadType = 0;

	// Codec (Attribute)
	// "a=rtpmap:96 H264/90000"
	FString Codec;
	uint32 ClockRate = 0;
	int32 Channels = 0;

	// Control URL (Attribute)
	// "a=control:track1" (relative)
	// "a=control:rtsp://host/path/track1" (absolute)
	FString ControlUrl;

	// Format Parameters (Attribute)
	// "a=fmtp: <params>"
	FString FormatParameters;

	// H.264 Specific Parameters
	TArray<uint8> SequenceParameterSet;
	TArray<uint8> PictureParameterSet;
	int32 PacketizationMode = 0;

	// AAC Specific Parameters
	TArray<uint8> AudioConfig;
};

struct FSdpSession
{
	// "s=<session_name>"
	FString SessionName;

	// "i=<session_info>"
	FString SessionInfo;

	// Provides the base URL of the stream content.
	// A track may be located at <ContentBase>/trackID=0
	FString ContentBaseUrl;

	TArray<FSdpMediaTrack> VideoTracks;
	TArray<FSdpMediaTrack> AudioTracks;

	bool HasVideo() const;
	bool HasAudio() const;
	const FSdpMediaTrack* GetFirstVideoTrack() const;
	const FSdpMediaTrack* GetFirstAudioTrack() const;
	FString ResolveControlUrl(const FString& InControlUrl) const;
};
