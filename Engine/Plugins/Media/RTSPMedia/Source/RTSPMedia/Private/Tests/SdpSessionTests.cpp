// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDP/SdpSession.h"

#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

// Aggregate control

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FResolveControlUrl_AggregateControl,
	"Plugins.RTSPMedia.SdpSession.ResolveControlUrl.AggregateControl.ReturnsContentBase",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::SmokeFilter)
bool FResolveControlUrl_AggregateControl::RunTest(const FString& Parameters)
{
	FSdpSession Session;
	Session.ContentBaseUrl = TEXT("rtsp://example.com:8554/stream/");
	TestEqual(TEXT("Aggregate control returns Content-Base"),
		Session.ResolveControlUrl(TEXT("*")),
		TEXT("rtsp://example.com:8554/stream/"));
	return true;
}

// Absolute URL

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FResolveControlUrl_AbsoluteUrl_RtspScheme,
	"Plugins.RTSPMedia.SdpSession.ResolveControlUrl.AbsoluteUrl.RtspScheme",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::SmokeFilter)
bool FResolveControlUrl_AbsoluteUrl_RtspScheme::RunTest(const FString& Parameters)
{
	FSdpSession Session;
	Session.ContentBaseUrl = TEXT("rtsp://example.com:8554/stream/");
	TestEqual(TEXT("Absolute URL returned as-is"),
		Session.ResolveControlUrl(TEXT("rtsp://other-server.com:554/track1")),
		TEXT("rtsp://other-server.com:554/track1"));
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FResolveControlUrl_AbsoluteUrl_HttpScheme,
	"Plugins.RTSPMedia.SdpSession.ResolveControlUrl.AbsoluteUrl.HttpScheme",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::SmokeFilter)
bool FResolveControlUrl_AbsoluteUrl_HttpScheme::RunTest(const FString& Parameters)
{
	FSdpSession Session;
	Session.ContentBaseUrl = TEXT("rtsp://example.com:8554/stream/");
	TestEqual(TEXT("HTTP scheme returned as-is"),
		Session.ResolveControlUrl(TEXT("http://example.com/track1")),
		TEXT("http://example.com/track1"));
	return true;
}

// Absolute path

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FResolveControlUrl_AbsolutePath_WithPath,
	"Plugins.RTSPMedia.SdpSession.ResolveControlUrl.AbsolutePath.ContentBaseWithPath",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::SmokeFilter)
bool FResolveControlUrl_AbsolutePath_WithPath::RunTest(const FString& Parameters)
{
	FSdpSession Session;
	Session.ContentBaseUrl = TEXT("rtsp://example.com:8554/stream/");
	TestEqual(TEXT("Absolute path resolves against server root"),
		Session.ResolveControlUrl(TEXT("/track1")),
		TEXT("rtsp://example.com:8554/track1"));
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FResolveControlUrl_AbsolutePath_NoTrailingSlash,
	"Plugins.RTSPMedia.SdpSession.ResolveControlUrl.AbsolutePath.ContentBaseNoTrailingSlash",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::SmokeFilter)
bool FResolveControlUrl_AbsolutePath_NoTrailingSlash::RunTest(const FString& Parameters)
{
	FSdpSession Session;
	Session.ContentBaseUrl = TEXT("rtsp://example.com:8554/stream");
	TestEqual(TEXT("Absolute path resolves against server root without trailing slash"),
		Session.ResolveControlUrl(TEXT("/track1")),
		TEXT("rtsp://example.com:8554/track1"));
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FResolveControlUrl_AbsolutePath_NoPath,
	"Plugins.RTSPMedia.SdpSession.ResolveControlUrl.AbsolutePath.ContentBaseNoPath",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::SmokeFilter)
bool FResolveControlUrl_AbsolutePath_NoPath::RunTest(const FString& Parameters)
{
	FSdpSession Session;
	Session.ContentBaseUrl = TEXT("rtsp://example.com:8554");
	TestEqual(TEXT("Absolute path appended to authority"),
		Session.ResolveControlUrl(TEXT("/track1")),
		TEXT("rtsp://example.com:8554/track1"));
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FResolveControlUrl_AbsolutePath_DeepPath,
	"Plugins.RTSPMedia.SdpSession.ResolveControlUrl.AbsolutePath.DeepAbsolutePath",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::SmokeFilter)
bool FResolveControlUrl_AbsolutePath_DeepPath::RunTest(const FString& Parameters)
{
	FSdpSession Session;
	Session.ContentBaseUrl = TEXT("rtsp://example.com:8554/live/stream/");
	TestEqual(TEXT("Deep absolute path still resolves against root"),
		Session.ResolveControlUrl(TEXT("/deep/path/track1")),
		TEXT("rtsp://example.com:8554/deep/path/track1"));
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FResolveControlUrl_AbsolutePath_NoPort,
	"Plugins.RTSPMedia.SdpSession.ResolveControlUrl.AbsolutePath.ContentBaseNoPort",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::SmokeFilter)
bool FResolveControlUrl_AbsolutePath_NoPort::RunTest(const FString& Parameters)
{
	FSdpSession Session;
	Session.ContentBaseUrl = TEXT("rtsp://example.com/stream/");
	TestEqual(TEXT("Absolute path resolves against portless server root"),
		Session.ResolveControlUrl(TEXT("/track1")),
		TEXT("rtsp://example.com/track1"));
	return true;
}

// Relative path

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FResolveControlUrl_RelativePath_TrailingSlash,
	"Plugins.RTSPMedia.SdpSession.ResolveControlUrl.RelativePath.ContentBaseTrailingSlash",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::SmokeFilter)
bool FResolveControlUrl_RelativePath_TrailingSlash::RunTest(const FString& Parameters)
{
	FSdpSession Session;
	Session.ContentBaseUrl = TEXT("rtsp://example.com:8554/stream/");
	TestEqual(TEXT("Relative path appended to Content-Base"),
		Session.ResolveControlUrl(TEXT("trackID=0")),
		TEXT("rtsp://example.com:8554/stream/trackID=0"));
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FResolveControlUrl_RelativePath_NoTrailingSlash,
	"Plugins.RTSPMedia.SdpSession.ResolveControlUrl.RelativePath.ContentBaseNoTrailingSlash",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::SmokeFilter)
bool FResolveControlUrl_RelativePath_NoTrailingSlash::RunTest(const FString& Parameters)
{
	FSdpSession Session;
	Session.ContentBaseUrl = TEXT("rtsp://example.com:8554/stream");
	TestEqual(TEXT("Relative path appended with separator"),
		Session.ResolveControlUrl(TEXT("trackID=0")),
		TEXT("rtsp://example.com:8554/stream/trackID=0"));
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FResolveControlUrl_RelativePath_SimpleTrackName,
	"Plugins.RTSPMedia.SdpSession.ResolveControlUrl.RelativePath.SimpleTrackName",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::SmokeFilter)
bool FResolveControlUrl_RelativePath_SimpleTrackName::RunTest(const FString& Parameters)
{
	FSdpSession Session;
	Session.ContentBaseUrl = TEXT("rtsp://example.com:8554/live/");
	TestEqual(TEXT("Simple track name appended"),
		Session.ResolveControlUrl(TEXT("video")),
		TEXT("rtsp://example.com:8554/live/video"));
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FResolveControlUrl_RelativePath_NoPath,
	"Plugins.RTSPMedia.SdpSession.ResolveControlUrl.RelativePath.ContentBaseNoPath",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::SmokeFilter)
bool FResolveControlUrl_RelativePath_NoPath::RunTest(const FString& Parameters)
{
	FSdpSession Session;
	Session.ContentBaseUrl = TEXT("rtsp://example.com:8554");
	TestEqual(TEXT("Relative path appended to authority-only Content-Base"),
		Session.ResolveControlUrl(TEXT("trackID=0")),
		TEXT("rtsp://example.com:8554/trackID=0"));
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FResolveControlUrl_RelativePath_NoPort,
	"Plugins.RTSPMedia.SdpSession.ResolveControlUrl.RelativePath.ContentBaseNoPort",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::SmokeFilter)
bool FResolveControlUrl_RelativePath_NoPort::RunTest(const FString& Parameters)
{
	FSdpSession Session;
	Session.ContentBaseUrl = TEXT("rtsp://example.com/stream/");
	TestEqual(TEXT("Relative path appended to portless Content-Base"),
		Session.ResolveControlUrl(TEXT("trackID=0")),
		TEXT("rtsp://example.com/stream/trackID=0"));
	return true;
}

#endif // WITH_DEV_AUTOMATION_TESTS
