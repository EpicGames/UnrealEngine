// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace RtspMedia::Default
{
	inline constexpr bool bAutoReconnect = true;
	inline constexpr bool bProvideCpuBuffer = true;
	inline constexpr int32 DecoderBufferSize = 5;
	inline constexpr int32 DecoderPollIntervalMs = 5;
	inline constexpr int32 DecoderLagThresholdMs = 1000;
	inline constexpr bool bJitterBufferAutoAdjust = true;
	inline constexpr uint32 JitterBufferDepthMs = 1000;
	inline constexpr float JitterBufferObservationWindowSeconds = 60.0f;
	inline constexpr int32 MaxFragmentBufferSizeMb = 8;
	inline constexpr int64 MaxFragmentBufferSizeBytes = MaxFragmentBufferSizeMb * 1024 * 1024;
	inline constexpr int32 MaxQueuedVideoSamples = 4;
	inline constexpr int32 MaxQueuedAudioSamples = 4;
	inline constexpr int32 MaxReconnectAttempts = 0;
	inline constexpr float MaxReconnectDelaySeconds = 30.0; 
	inline constexpr float MinReconnectDelaySeconds = 1.0;
	inline constexpr int32 Port = 8554;
	inline constexpr float RequestTimeoutSeconds = 5.0;
	inline constexpr int32 SocketBufferSizeKb = 512;
	inline constexpr int32 SocketBufferSizeBytes = SocketBufferSizeKb * 1024;
}
