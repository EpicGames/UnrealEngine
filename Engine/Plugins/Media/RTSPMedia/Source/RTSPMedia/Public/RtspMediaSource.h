// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseMediaSource.h"

#include "RtspMediaDefaults.h"

#include "RtspMediaSource.generated.h"

UENUM(BlueprintType)
enum class ERtspMediaTransportProtocol : uint8
{
	/** RTP packets are interleaved with RTSP control messages */
	TCP
};

namespace RtspMedia::Default
{
	inline constexpr ERtspMediaTransportProtocol TransportProtocol = ERtspMediaTransportProtocol::TCP;
}

UCLASS(BlueprintType, hideCategories=(Platforms, Object), meta=(DisplayName="RTSP Media Source"))
class RTSPMEDIA_API URtspMediaSource : public UBaseMediaSource
{
	GENERATED_BODY()

public:

	/** Host name or IP address of the RTSP server */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Server")
	FString Host;

	/** Port number of the RTSP Server */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Server", meta=(ClampMin="1", ClampMax="65535"))
	int32 Port = RtspMedia::Default::Port;

	/** Path to the stream on RTSP server */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Server")
	FString Path;

	/** Only TCP interleaved transport is supported */
	UPROPERTY(BlueprintReadOnly, Category="Client", meta=(Hidden))
	ERtspMediaTransportProtocol TransportProtocol = RtspMedia::Default::TransportProtocol;
	
	/** Socket buffer size for stream data in KB.
	 * Increase this setting for high bitrate streams (e.g. 4K) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Client", meta=(DisplayName="Socket Buffer Size (KB)", ClampMin="128", ClampMax="4096"))
	int32 SocketBufferSizeKb = RtspMedia::Default::SocketBufferSizeKb;
	
	/** RTSP request timeout in seconds.
	 * Increase for slow or unreliable networks to avoid premature session failures */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Client", meta=(DisplayName="Request Timeout (s)", ClampMin="1.0", ClampMax="30.0"))
	float RequestTimeoutSeconds = RtspMedia::Default::RequestTimeoutSeconds;
	
	/** Automatically reconnect when the connection fails
	 * The player will begin reconnection attempts after the delay specified in Min Retry Delay */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Client")
	bool bAutoReconnect = RtspMedia::Default::bAutoReconnect;

	/** The minimum amount of time to wait before attempting to reconnect after a connection failure */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Client", meta=(DisplayName="Min Reconnect Delay (s)", EditCondition="bAutoReconnect", ClampMin="1.0", ClampMax="60.0"))
	float MinReconnectDelaySeconds = RtspMedia::Default::MinReconnectDelaySeconds;

	/** The maximum amount of time to wait before attempting to reconnect after a connection failure
	 * The player will gradually increase the delay before each connection attempt but for no longer than this maximum */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Client", meta=(DisplayName="Max Reconnect Delay (s)", EditCondition="bAutoReconnect", ClampMin="1.0", ClampMax="300.0"))
	float MaxReconnectDelaySeconds = RtspMedia::Default::MaxReconnectDelaySeconds;

	/** The maximum number of times the player will attempt to reconnect after a connection failure
	 * Set to 0 to retry indefinitely */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Client", meta=(EditCondition="bAutoReconnect", ClampMin="0", ClampMax="1000"))
	int32 MaxReconnectAttempts = RtspMedia::Default::MaxReconnectAttempts;
	
	/** The jitter buffer accounts for spikes in the network delivery of RTP packets in order to provide a steady stream of packets to the decoder. 
	 * This setting automatically adjusts the jitter buffer depth based on network conditions.
	 * When disabled, the static Depth value is used. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Jitter Buffer", meta=(DisplayName="Auto Adjust"))
	bool bJitterBufferAutoAdjust = RtspMedia::Default::bJitterBufferAutoAdjust;

	/** A higher buffer depth will absorb more network jitter at the cost of additional latency.
	 * On clean networks a low or zero value may be configured which will minimize the latency introduced by the buffer. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Jitter Buffer", meta=(DisplayName="Depth (ms)", EditCondition="!bJitterBufferAutoAdjust", EditConditionHides, ClampMin="0", ClampMax="5000"))
	int32 JitterBufferDepthMs = RtspMedia::Default::JitterBufferDepthMs;

	/** How long network conditions must remain stable before the jitter buffer depth is reduced.
	 * Longer windows provide more protection against intermittent spikes. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Jitter Buffer", meta=(DisplayName="Observation Window (s)", EditCondition="bJitterBufferAutoAdjust", EditConditionHides, ClampMin="30.0", ClampMax="300.0"))
	float JitterBufferObservationWindowSeconds = RtspMedia::Default::JitterBufferObservationWindowSeconds;
	
	/** Places a limit on how much memory is used for fragmented NAL unit reassembly
	 * NAL units will be dropped if this memory limit is exceeded
	 * Consider increasing this value for high bitrate streams */ 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Depacketization", meta=(DisplayName="Max Fragment Buffer Size (MB)", ClampMin="2", ClampMax="64"))
	int32 MaxFragmentBufferSizeMb = RtspMedia::Default::MaxFragmentBufferSizeMb;

	/** Decoder reorder buffer size. Use 1 for minimum latency (i.e. no B-frames)
	 * Increase if video frames appears out of order */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Decoder", meta=(DisplayName="Buffer Size (frames)", ClampMin="1", ClampMax="16"))
	int32 DecoderBufferSize = RtspMedia::Default::DecoderBufferSize;

	/** How frequently to poll the decoder for output frames (in milliseconds)
	 * Lower values reduce latency but increase CPU usage */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Decoder", meta=(DisplayName="Poll Interval (ms)", ClampMin="1", ClampMax="10"))
	int32 DecoderPollIntervalMs = RtspMedia::Default::DecoderPollIntervalMs;

	/** Maximum decoded video frames to queue while waiting for game thread consumption
	 *  Higher values reduce visible skips after game thread hitches at the cost of memory */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Decoder", meta=(ClampMin="2", ClampMax="16"))
	int32 MaxQueuedVideoSamples = RtspMedia::Default::MaxQueuedVideoSamples;

	/** Populate a CPU-accessible pixel buffer alongside the GPU texture
	 * Whether this is available depends on the underlying decoder implementation */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Decoder", meta=(DisplayName="Provide CPU Buffer"))
	bool bProvideCpuBuffer = RtspMedia::Default::bProvideCpuBuffer;

	//~ Begin UMediaSource
	virtual FString GetUrl() const override;
	virtual bool Validate() const override;
	//~ End UMediaSource

	//~ Begin IMediaOptions
	virtual bool GetMediaOption(const FName& InKey, bool InDefaultValue) const override;
	virtual int64 GetMediaOption(const FName& InKey, int64 InDefaultValue) const override;
	virtual double GetMediaOption(const FName& InKey, double InDefaultValue) const override;
	virtual bool HasMediaOption(const FName& InKey) const override;
	//~ End IMediaOptions

	//~ Begin UBaseMediaSource
	virtual FName GetDesiredPlayerName() const override;
	//~ End UBaseMediaSource
};