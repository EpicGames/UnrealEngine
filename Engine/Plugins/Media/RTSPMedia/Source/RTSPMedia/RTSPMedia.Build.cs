// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

public class RTSPMedia : ModuleRules
{
	public RTSPMedia(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.NoPCHs;
		bRequiresPlatformSDK = true;
		
		PublicDependencyModuleNames.AddRange(new string[]
		{
			"Core",
			"MediaAssets"
		});

		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"CoreUObject",
			"ElectraBase",
			"ElectraDecoders",
			"ElectraCodecFactory",
			"ElectraSamples",
			"Engine",
			"Media",
			"MediaIOCore",
			"MediaUtils",
			"Networking",
			"RenderCore",
			"RHI",
			"Slate",
			"SlateCore",
			"Sockets",
			"TimeManagement"
		});

		if (Target.IsInPlatformGroup(UnrealPlatformGroup.Apple))
		{
			PublicFrameworks.Add("CoreVideo");
		}
	}
}