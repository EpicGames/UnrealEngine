// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "AssetDefinition/AssetDefinition_MediaSource.h"
#include "RtspMediaSource.h"
#include "AssetDefinition_RtspMediaSource.generated.h"

UCLASS()
class UAssetDefinition_RtspMediaSource : public UAssetDefinition_MediaSource
{
	GENERATED_BODY()
public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "AssetDefinition_RtspMediaSource", "RTSP Media Source"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(0, 180, 160)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return URtspMediaSource::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Media, NSLOCTEXT("AssetDefinition", "AssetDefinition_RtspMediaSourceSubMenu", "Media Sources + Outputs"), ECategoryMenuType::Section) };
		return Categories;
	}
	// UAssetDefinition End
};
