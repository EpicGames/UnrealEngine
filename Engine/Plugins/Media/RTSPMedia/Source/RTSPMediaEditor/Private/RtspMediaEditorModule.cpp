// Copyright Epic Games, Inc. All Rights Reserved.

#include "Interfaces/IPluginManager.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"

DEFINE_LOG_CATEGORY_STATIC(LogRtspMediaEditor, Log, All);

#define LOCTEXT_NAMESPACE "RtspMediaEditorModule"

class FRtspMediaEditorModule : public IModuleInterface
{
public:
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

private:
	void RegisterStyle();
	void UnregisterStyle();

	TUniquePtr<FSlateStyleSet> StyleInstance;
};

void FRtspMediaEditorModule::StartupModule()
{
	RegisterStyle();
}

void FRtspMediaEditorModule::ShutdownModule()
{
	UnregisterStyle();
}

void FRtspMediaEditorModule::RegisterStyle()
{
	const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(TEXT("RTSPMedia"));

	if (!Plugin.IsValid())
	{
		return;
	}

	const FString ContentDirectory = Plugin->GetContentDir();

	StyleInstance = MakeUnique<FSlateStyleSet>("RtspMediaStyle");
	StyleInstance->SetContentRoot(ContentDirectory / TEXT("Editor/Icons"));

	StyleInstance->Set(
		"ClassIcon.RtspMediaSource",
		new FSlateVectorImageBrush(StyleInstance->RootToContentDir(TEXT("RtspMediaSource_16"), TEXT(".svg")), CoreStyleConstants::Icon16x16)
	);

	StyleInstance->Set(
		"ClassThumbnail.RtspMediaSource", 
		new FSlateVectorImageBrush(StyleInstance->RootToContentDir(TEXT("RtspMediaSource_64"), TEXT(".svg")), CoreStyleConstants::Icon64x64)
	);

	FSlateStyleRegistry::RegisterSlateStyle(*StyleInstance);
}

void FRtspMediaEditorModule::UnregisterStyle()
{
	if (StyleInstance.IsValid())
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*StyleInstance);
		StyleInstance.Reset();
	}
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FRtspMediaEditorModule, RTSPMediaEditor)
