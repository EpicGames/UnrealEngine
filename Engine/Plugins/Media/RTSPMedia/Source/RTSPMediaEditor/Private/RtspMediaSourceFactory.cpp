// Copyright Epic Games, Inc. All Rights Reserved.

#include "RtspMediaSourceFactory.h"

#include "AssetTypeCategories.h"
#include "RtspMediaSource.h"

URtspMediaSourceFactory::URtspMediaSourceFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = URtspMediaSource::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}

UObject* URtspMediaSourceFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags InFlags, UObject* InContext, FFeedbackContext* InWarn)
{
	return NewObject<URtspMediaSource>(InParent, InClass, InName, InFlags);
}

uint32 URtspMediaSourceFactory::GetMenuCategories() const
{
	return EAssetTypeCategories::Media;
}

bool URtspMediaSourceFactory::ShouldShowInNewMenu() const
{
	return true;
}