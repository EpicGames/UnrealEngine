// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class RTSPMediaEditor : ModuleRules
{
    public RTSPMediaEditor(ReadOnlyTargetRules Target) : base(Target)
    {
        PrivateDependencyModuleNames.AddRange(new string[]
        {
            "AssetDefinition",
            "Core",
            "CoreUObject",
            "MediaPlayerEditor",
            "Projects",
            "RTSPMedia",
            "Slate",
            "SlateCore",
            "UnrealEd"
        });
    }
}